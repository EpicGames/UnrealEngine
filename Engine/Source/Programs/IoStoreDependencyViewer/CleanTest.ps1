# Clean up test directory
$TestDir = "C:\Users\daniel.lamb\AppData\Local\Temp\IoStoreDependencyViewer_Test"

# Kill any IoStoreDependencyViewer processes
Get-Process | Where-Object { $_.ProcessName -eq "IoStoreDependencyViewer" } | Stop-Process -Force -ErrorAction SilentlyContinue

Start-Sleep -Seconds 2

# Remove test directory
if (Test-Path $TestDir) {
    Remove-Item $TestDir -Recurse -Force -ErrorAction Stop
    Write-Host "Test directory cleaned" -ForegroundColor Green
} else {
    Write-Host "Test directory doesn't exist" -ForegroundColor Yellow
}
