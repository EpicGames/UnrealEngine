# Use Windows LoadLibrary API to detect missing DLLs
param(
    [string]$TestDir = "C:\Users\daniel.lamb\AppData\Local\Temp\IoStoreDependencyViewer_Test"
)

$signature = @'
[DllImport("kernel32.dll", SetLastError=true, CharSet=CharSet.Unicode)]
public static IntPtr LoadLibraryEx(string lpFileName, IntPtr hReservedNull, uint dwFlags);

[DllImport("kernel32.dll", SetLastError=true)]
[return: MarshalAs(UnmanagedType.Bool)]
public static extern bool FreeLibrary(IntPtr hModule);

[DllImport("kernel32.dll")]
public static extern uint GetLastError();
'@

Add-Type -MemberDefinition $signature -Name NativeMethods -Namespace Win32

$LOAD_LIBRARY_AS_DATAFILE = 0x00000002
$LOAD_LIBRARY_AS_IMAGE_RESOURCE = 0x00000020

$exePath = Join-Path $TestDir "IoStoreDependencyViewer.exe"

if (-not (Test-Path $exePath)) {
    Write-Host "Error: $exePath not found" -ForegroundColor Red
    exit 1
}

Write-Host "========================================" -ForegroundColor Cyan
Write-Host "Testing DLL Dependencies" -ForegroundColor Cyan
Write-Host "========================================" -ForegroundColor Cyan
Write-Host ""
Write-Host "Test Directory: $TestDir" -ForegroundColor White
Write-Host ""

# Set current directory to test location so relative DLL loading works
[System.IO.Directory]::SetCurrentDirectory($TestDir)

# Get all DLLs in the package
$packagedDlls = Get-ChildItem (Join-Path $TestDir "*.dll") | Select-Object -ExpandProperty Name

Write-Host "Packaged DLLs: $($packagedDlls.Count)" -ForegroundColor White
Write-Host ""

Write-Host "Attempting to load each DLL..." -ForegroundColor Cyan
Write-Host ""

$failedDlls = @()
$loadedHandles = @()

foreach ($dll in $packagedDlls) {
    $dllPath = Join-Path $TestDir $dll

    # Try to load the DLL
    $handle = [Win32.NativeMethods]::LoadLibraryEx($dllPath, [IntPtr]::Zero, $LOAD_LIBRARY_AS_DATAFILE)

    if ($handle -eq [IntPtr]::Zero) {
        $errorCode = [Win32.NativeMethods]::GetLastError()
        Write-Host "  [FAIL] $dll (Error: $errorCode)" -ForegroundColor Red
        $failedDlls += @{ DLL = $dll; ErrorCode = $errorCode }
    } else {
        Write-Host "  [OK] $dll" -ForegroundColor Green
        $loadedHandles += $handle
    }
}

# Clean up loaded DLLs
foreach ($handle in $loadedHandles) {
    [Win32.NativeMethods]::FreeLibrary($handle) | Out-Null
}

Write-Host ""

if ($failedDlls.Count -gt 0) {
    Write-Host "====================================" -ForegroundColor Red
    Write-Host "FAILED TO LOAD $($failedDlls.Count) DLL(S)" -ForegroundColor Red
    Write-Host "====================================" -ForegroundColor Red
    Write-Host ""

    foreach ($failed in $failedDlls) {
        Write-Host "  $($failed.DLL)" -ForegroundColor Red

        # Error code meanings
        $errorMsg = switch ($failed.ErrorCode) {
            126 { "The specified module could not be found (missing dependency)" }
            127 { "The specified procedure could not be found" }
            193 { "Not a valid Win32 application" }
            default { "Unknown error" }
        }
        Write-Host "      Error $($failed.ErrorCode): $errorMsg" -ForegroundColor Yellow
    }

    Write-Host ""
    Write-Host "Error 126 typically means a dependency DLL is missing." -ForegroundColor Yellow
    Write-Host ""

    exit 1
} else {
    Write-Host "====================================" -ForegroundColor Green
    Write-Host "ALL DLLs LOADED SUCCESSFULLY" -ForegroundColor Green
    Write-Host "====================================" -ForegroundColor Green
    exit 0
}
