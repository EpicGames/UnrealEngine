// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IoStoreDependencyViewer : ModuleRules
{
	public IoStoreDependencyViewer(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePaths.Add("Runtime/Launch/Public");
		PrivateIncludePaths.Add("Runtime/Launch/Private");

		// Add internal include paths for IoStore
		PrivateIncludePaths.Add("Runtime/Core/Internal");
		PrivateIncludePaths.Add("Runtime/Experimental/IoStore/OnDemandCore/Internal");

		// Enable exceptions for IoStoreOnDemandUtilities
		bEnableExceptions = true;

		PrivateDependencyModuleNames.AddRange(new string[] {
			"ApplicationCore",
			"Core",
			"CoreUObject",
			"DesktopPlatform",
			"HTTP",
			"InputCore",
			"IoStoreOnDemand",
			"IoStoreOnDemandUtilities",
			"Json",
			"JsonUtilities",
			"PakFile",
			"Projects",
			"RSA",
			"Slate",
			"SlateCore",
			"StandaloneRenderer",
		});
	}
}
