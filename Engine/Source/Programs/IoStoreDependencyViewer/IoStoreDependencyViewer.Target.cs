// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.Desktop)]
public class IoStoreDependencyViewerTarget : TargetRules
{
	public IoStoreDependencyViewerTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;
		LinkType = TargetLinkType.Monolithic;
		LaunchModuleName = "IoStoreDependencyViewer";

		// Compile against CoreUObject for IoStore support
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = true;
		bCompileAgainstApplicationCore = true;

		// Console application with GUI
		bIsBuildingConsoleApplication = true;

		// Enable logging
		bUseLoggingInShipping = true;

		// No plugins needed
		bCompileWithPluginSupport = false;
		bIncludePluginsForTargetPlatforms = false;

		// Enable exceptions for IoStoreOnDemandUtilities
		bForceEnableExceptions = true;
	}
}
