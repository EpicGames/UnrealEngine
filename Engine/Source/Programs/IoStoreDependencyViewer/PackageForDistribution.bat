@echo off
REM IoStoreDependencyViewer Distribution Packaging Script

echo.
echo ========================================
echo  IoStoreDependencyViewer Packager
echo ========================================
echo.
echo Reading settings from PackageSettings.ini...
echo.

REM Run the PowerShell script
powershell.exe -ExecutionPolicy Bypass -File "%~dp0PackageForDistribution.ps1"

if %ERRORLEVEL% EQU 0 (
    echo.
    echo SUCCESS!
    pause
) else (
    echo.
    echo FAILED! Check the error messages above.
    pause
    exit /b 1
)
