# IoStoreDependencyViewer Distribution Packaging Script
# This script bundles the IoStoreDependencyViewer into a standalone distributable package

$ErrorActionPreference = "Stop"

# Paths
$EngineRoot = Split-Path -Parent (Split-Path -Parent (Split-Path -Parent $PSScriptRoot))
$BinariesDir = Join-Path $EngineRoot "Binaries\Win64"
$ExePath = Join-Path $BinariesDir "IoStoreDependencyViewer.exe"
$SettingsPath = Join-Path $PSScriptRoot "PackageSettings.ini"

Write-Host "=== IoStoreDependencyViewer Distribution Packager ===" -ForegroundColor Cyan
Write-Host ""

# Read settings from INI file
if (-not (Test-Path $SettingsPath)) {
    Write-Error "PackageSettings.ini not found at: $SettingsPath"
    Write-Host "Please create PackageSettings.ini with the following format:" -ForegroundColor Yellow
    Write-Host "[Packaging]" -ForegroundColor Yellow
    Write-Host "OutputPath=D:\Path\To\Output" -ForegroundColor Yellow
    Write-Host "Version=1.0.0" -ForegroundColor Yellow
    exit 1
}

Write-Host "Reading settings from: $SettingsPath" -ForegroundColor Cyan

# Parse INI file
$IniContent = Get-Content $SettingsPath
$OutputDir = $null
$Version = $null
$IncludeZenTools = $false
$ZenReleasePath = $null

foreach ($Line in $IniContent) {
    $Line = $Line.Trim()
    if ($Line -match '^OutputPath=(.+)$') {
        $OutputDir = $Matches[1].Trim()
    }
    if ($Line -match '^Version=(.+)$') {
        $Version = $Matches[1].Trim()
    }
    if ($Line -match '^IncludeZenTools=(.+)$') {
        $IncludeZenTools = $Matches[1].Trim() -eq 'true'
    }
    if ($Line -match '^ZenReleasePath=(.+)$') {
        $ZenReleasePath = $Matches[1].Trim()
    }
}

if (-not $OutputDir) {
    Write-Error "OutputPath not found in PackageSettings.ini"
    exit 1
}

if (-not $Version) {
    Write-Error "Version not found in PackageSettings.ini"
    exit 1
}

Write-Host "  Output Directory: $OutputDir" -ForegroundColor Green
Write-Host "  Version: $Version" -ForegroundColor Green
Write-Host "  Include Zen Tools: $IncludeZenTools" -ForegroundColor $(if ($IncludeZenTools) { "Green" } else { "Yellow" })
if ($IncludeZenTools -and $ZenReleasePath) {
    Write-Host "  Zen Release Path: $ZenReleasePath" -ForegroundColor Green
}
Write-Host ""

# Output paths - fixed filename
# Use D: drive for temp to avoid C: drive space issues
$TempDir = "D:\Temp\IoStoreDependencyViewer_Package"
$ZipName = "IoStoreDependencyViewer.zip"
$ZipPath = Join-Path $OutputDir $ZipName

# Verify exe exists
if (-not (Test-Path $ExePath)) {
    Write-Error "IoStoreDependencyViewer.exe not found at: $ExePath"
    Write-Host "Please build the project first using: Build.bat IoStoreDependencyViewer Win64 Development" -ForegroundColor Yellow
    exit 1
}

Write-Host "Found IoStoreDependencyViewer.exe" -ForegroundColor Green
Write-Host "  Path: $ExePath"
Write-Host "  Size: $((Get-Item $ExePath).Length / 1MB | ForEach-Object { "{0:N2} MB" -f $_ })"
Write-Host ""

# Clean and create temp directory
if (Test-Path $TempDir) {
    Remove-Item $TempDir -Recurse -Force
}
New-Item -ItemType Directory -Path $TempDir | Out-Null

Write-Host "Copying files to temporary directory..." -ForegroundColor Cyan

# Create Binaries/Win64 directory structure to match engine layout
$BinDir = Join-Path $TempDir "Engine\Binaries\Win64"
New-Item -ItemType Directory -Path $BinDir -Force | Out-Null

# Copy the executable
Copy-Item $ExePath -Destination $BinDir

# Create version.txt
$ZenToolsInfo = if ($IncludeZenTools) {
    "`n`nExperimental zen.exe included in Engine\Binaries\Win64\ (partial download support)"
} else {
    ""
}

$VersionContent = @"
IoStoreDependencyViewer
Version: $Version
Build Date: $(Get-Date -Format 'yyyy-MM-dd HH:mm:ss')

A standalone tool for analyzing and visualizing IoStore container dependencies in Unreal Engine projects.

For documentation and usage instructions, see README.md

To run: Execute Engine\Binaries\Win64\IoStoreDependencyViewer.exe$ZenToolsInfo
"@

$VersionPath = Join-Path $TempDir "version.txt"
Set-Content -Path $VersionPath -Value $VersionContent
Write-Host "  [OK] version.txt" -ForegroundColor Green

# Copy README if it exists
$ReadmePath = Join-Path $PSScriptRoot "README_DISTRIBUTION.md"
if (Test-Path $ReadmePath) {
    Copy-Item $ReadmePath -Destination (Join-Path $TempDir "README.md")
    Write-Host "  [OK] README.md" -ForegroundColor Green
}

# Copy ICU Internationalization data (required for text rendering)
Write-Host ""
Write-Host "Copying Engine content..." -ForegroundColor Cyan

$IcuSourcePath = Join-Path $EngineRoot "Content\Internationalization"
$IcuDestPath = Join-Path $TempDir "Engine\Content\Internationalization"

if (Test-Path $IcuSourcePath) {
    New-Item -ItemType Directory -Path $IcuDestPath -Force | Out-Null
    Copy-Item -Path "$IcuSourcePath\*" -Destination $IcuDestPath -Recurse -Force
    $icuSize = (Get-ChildItem $IcuDestPath -Recurse | Measure-Object -Property Length -Sum).Sum / 1MB
    Write-Host "  [OK] ICU Internationalization data ($("{0:N2}" -f $icuSize) MB)" -ForegroundColor Green
} else {
    Write-Host "  [SKIP] ICU data not found" -ForegroundColor Yellow
}

# Copy Slate content (UI resources)
$SlateSourcePath = Join-Path $EngineRoot "Content\Slate"
$SlateDestPath = Join-Path $TempDir "Engine\Content\Slate"

if (Test-Path $SlateSourcePath) {
    New-Item -ItemType Directory -Path $SlateDestPath -Force | Out-Null
    Copy-Item -Path "$SlateSourcePath\*" -Destination $SlateDestPath -Recurse -Force
    $slateSize = (Get-ChildItem $SlateDestPath -Recurse | Measure-Object -Property Length -Sum).Sum / 1MB
    Write-Host "  [OK] Slate UI content ($("{0:N2}" -f $slateSize) MB)" -ForegroundColor Green
} else {
    Write-Host "  [SKIP] Slate content not found" -ForegroundColor Yellow
}

# Copy Shaders (required for rendering)
$ShaderSourcePath = Join-Path $EngineRoot "Shaders\StandaloneRenderer"
$ShaderDestPath = Join-Path $TempDir "Engine\Shaders\StandaloneRenderer"

if (Test-Path $ShaderSourcePath) {
    New-Item -ItemType Directory -Path $ShaderDestPath -Force | Out-Null
    Copy-Item -Path "$ShaderSourcePath\*" -Destination $ShaderDestPath -Recurse -Force
    $shaderSize = (Get-ChildItem $ShaderDestPath -Recurse | Measure-Object -Property Length -Sum).Sum / 1MB
    Write-Host "  [OK] Shaders ($("{0:N2}" -f $shaderSize) MB)" -ForegroundColor Green
} else {
    Write-Host "  [SKIP] Shaders not found" -ForegroundColor Yellow
}

# Required DLL modules based on Build.cs
$RequiredModules = @(
    "UnrealEditor-ApplicationCore.dll",
    "UnrealEditor-Core.dll",
    "UnrealEditor-CoreUObject.dll",
    "UnrealEditor-DesktopPlatform.dll",
    "UnrealEditor-HTTP.dll",
    "UnrealEditor-InputCore.dll",
    "UnrealEditor-IoStoreOnDemand.dll",
    "UnrealEditor-IoStoreOnDemandUtilities.dll",
    "UnrealEditor-Json.dll",
    "UnrealEditor-JsonUtilities.dll",
    "UnrealEditor-PakFile.dll",
    "UnrealEditor-Projects.dll",
    "UnrealEditor-RSA.dll",
    "UnrealEditor-Slate.dll",
    "UnrealEditor-SlateCore.dll",
    "UnrealEditor-StandaloneRenderer.dll",
    # Additional core dependencies
    "UnrealEditor-RenderCore.dll",
    "UnrealEditor-RHI.dll",
    "UnrealEditor-ImageCore.dll",
    "UnrealEditor-AppFramework.dll",
    "UnrealEditor-Sockets.dll",
    "UnrealEditor-Networking.dll",
    "UnrealEditor-SSL.dll",
    "UnrealEditor-TraceLog.dll",
    "UnrealEditor-BuildSettings.dll",
    # D3D/Rendering
    "UnrealEditor-D3D12RHI.dll",
    "UnrealEditor-D3D11RHI.dll",
    # Windows platform
    "UnrealEditor-WindowsD3D.dll",
    # Compression
    "oo2core_5_win64.dll",
    "oo2core_9_win64.dll",
    # Intel Threading Building Blocks (TBB) - Memory allocator
    "tbb12.dll",
    "tbbmalloc.dll"
)

$CopiedCount = 0
$MissingModules = @()

foreach ($Module in $RequiredModules) {
    $SourcePath = Join-Path $BinariesDir $Module
    if (Test-Path $SourcePath) {
        Copy-Item $SourcePath -Destination $BinDir
        $CopiedCount++
        Write-Host "  [OK] $Module" -ForegroundColor Green
    } else {
        $MissingModules += $Module
        Write-Host "  [SKIP] $Module (not found)" -ForegroundColor Yellow
    }
}

Write-Host ""
Write-Host "Copied $CopiedCount dependency files" -ForegroundColor Green
if ($MissingModules.Count -gt 0) {
    Write-Host "Skipped $($MissingModules.Count) missing modules (may not be required)" -ForegroundColor Yellow
}
Write-Host ""

# Copy experimental zen.exe if enabled
if ($IncludeZenTools) {
    Write-Host "Including experimental zen.exe..." -ForegroundColor Cyan

    if (-not $ZenReleasePath) {
        Write-Warning "IncludeZenTools=true but ZenReleasePath not specified in PackageSettings.ini"
        Write-Host "  [SKIP] zen.exe" -ForegroundColor Yellow
    } elseif (-not (Test-Path $ZenReleasePath)) {
        Write-Warning "ZenReleasePath does not exist: $ZenReleasePath"
        Write-Host "  [SKIP] zen.exe" -ForegroundColor Yellow
    } else {
        # Copy zen.exe to same directory as IoStoreDependencyViewer.exe
        $ZenExePath = Join-Path $ZenReleasePath "zen.exe"
        if (Test-Path $ZenExePath) {
            Copy-Item $ZenExePath -Destination $BinDir
            $zenSize = (Get-Item $ZenExePath).Length / 1MB
            Write-Host "  [OK] zen.exe ($("{0:N2} MB" -f $zenSize)) - experimental build with partial download support" -ForegroundColor Green
        } else {
            Write-Host "  [SKIP] zen.exe not found at: $ZenExePath" -ForegroundColor Yellow
        }
    }
    Write-Host ""
} else {
    Write-Host "Zen.exe not included (IncludeZenTools=false)" -ForegroundColor Yellow
    Write-Host "  Users must provide their own zen.exe for cloud downloads" -ForegroundColor Yellow
    Write-Host ""
}

# Create output directory if it doesn't exist
if (-not (Test-Path $OutputDir)) {
    New-Item -ItemType Directory -Path $OutputDir | Out-Null
    Write-Host "Created output directory: $OutputDir" -ForegroundColor Green
}

# Remove existing zip if it exists
if (Test-Path $ZipPath) {
    Remove-Item $ZipPath -Force
    Write-Host "Removed existing package" -ForegroundColor Yellow
}

# Create zip file
Write-Host "Creating distribution package (this may take a few minutes due to ICU data size)..." -ForegroundColor Cyan
Write-Host "  Compressing..." -ForegroundColor Yellow

try {
    # Use fastest compression to speed up the process
    Compress-Archive -Path "$TempDir\*" -DestinationPath $ZipPath -CompressionLevel Fastest
    Write-Host "  [OK] Compression complete" -ForegroundColor Green
} catch {
    Write-Error "Failed to create archive: $_"
    exit 1
}

# Clean up temp directory
Remove-Item $TempDir -Recurse -Force

# Display results
$ZipSize = (Get-Item $ZipPath).Length / 1MB
$ZenToolsStatus = if ($IncludeZenTools) { "Included" } else { "Not Included" }
Write-Host ""
Write-Host "=== Package Created Successfully ===" -ForegroundColor Green
Write-Host "  Output: $ZipPath"
Write-Host "  Size: $("{0:N2} MB" -f $ZipSize)"
Write-Host "  Version: $Version"
Write-Host "  Files: IoStoreDependencyViewer.exe + $CopiedCount DLLs + version.txt + README.md"
Write-Host "  Zen Tools: $ZenToolsStatus" -ForegroundColor $(if ($IncludeZenTools) { "Green" } else { "Yellow" })
Write-Host ""
Write-Host "Package is ready for distribution!" -ForegroundColor Cyan
