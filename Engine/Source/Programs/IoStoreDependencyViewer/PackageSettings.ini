[Packaging]
# Output directory where the IoStoreDependencyViewer.zip will be created
OutputPath=D:\Depot\depot\InternalTools\UnrealGameSync\UnrealGameSync\bin\Tools\IoStoreDependencyViewer

# Version number (will be written to version.txt inside the package)
Version=1.0.6

# Include experimental zen.exe with partial download support (optional)
# Set to true to include zen.exe and OidcToken.exe in the package
# Set to false or comment out to exclude zen tools (users must provide their own)
IncludeZenTools=true

# Path to experimental zen.exe release build (only used if IncludeZenTools=true)
ZenReleasePath=D:\github\zen\Zen\build\windows\x64\release
