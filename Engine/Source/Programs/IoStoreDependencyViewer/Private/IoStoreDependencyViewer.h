// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogIoStoreDependencyViewer, Log, All);

// Test function for verifying .uondemandtoc loading
void TestOnDemandTocLoading(const FString& DirectoryPath);

// Test function for verifying cloud download
void TestCloudDownload(const FString& DownloadPath);

// Test function for downloading specific build from cloud
void TestCloudDownloadSpecific(const FString& Namespace, const FString& Bucket, const FString& BuildId, const FString& DownloadPath);

// Test function for exporting TOC assets to CSV
void TestTocToCSVFromCommandLine();

// Baseline test: Downloads full .utoc + .ucas files and exports asset data to CSV
bool RunTestBaselineUcas();

// Cloud partial download test: Downloads only metadata, uses on-demand fetching, exports to CSV
bool RunTestCloudPartialDownloadUCas();

// Zen Reader Integration test: Downloads metadata, uses FIoStoreReaderZenBuild, exports to CSV
bool RunTestZenReaderIntegration();

// Test function for partial download functionality
void TestPartialDownload();
