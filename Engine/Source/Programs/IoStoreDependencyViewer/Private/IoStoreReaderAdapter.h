// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IO/IoDispatcher.h"
#include "IoStoreReaderZenBuild.h"

/**
 * Adapter that wraps either a standard FIoStoreReader or a FIoStoreReaderZenBuild
 * Provides a unified interface for both types
 */
class FIoStoreReaderAdapter
{
public:
	enum class EReaderType
	{
		Standard,	// File-based FIoStoreReader
		ZenBuild	// Cloud-based FIoStoreReaderZenBuild
	};

	// Construct with standard reader
	explicit FIoStoreReaderAdapter(TUniquePtr<FIoStoreReader> InStandardReader)
		: StandardReader(MoveTemp(InStandardReader))
		, ReaderType(EReaderType::Standard)
	{
	}

	// Construct with zen build reader
	explicit FIoStoreReaderAdapter(TUniquePtr<FIoStoreReaderZenBuild> InZenReader)
		: ZenReader(MoveTemp(InZenReader))
		, ReaderType(EReaderType::ZenBuild)
	{
	}

	// IoStore interface - delegates to the appropriate reader
	FIoContainerId GetContainerId() const
	{
		return ReaderType == EReaderType::Standard
			? StandardReader->GetContainerId()
			: ZenReader->GetContainerId();
	}

	uint32 GetVersion() const
	{
		return ReaderType == EReaderType::Standard
			? StandardReader->GetVersion()
			: ZenReader->GetVersion();
	}

	EIoContainerFlags GetContainerFlags() const
	{
		return ReaderType == EReaderType::Standard
			? StandardReader->GetContainerFlags()
			: ZenReader->GetContainerFlags();
	}

	FGuid GetEncryptionKeyGuid() const
	{
		return ReaderType == EReaderType::Standard
			? StandardReader->GetEncryptionKeyGuid()
			: ZenReader->GetEncryptionKeyGuid();
	}

	FString GetContainerName() const
	{
		return ReaderType == EReaderType::Standard
			? StandardReader->GetContainerName()
			: ZenReader->GetContainerName();
	}

	int32 GetChunkCount() const
	{
		return ReaderType == EReaderType::Standard
			? StandardReader->GetChunkCount()
			: ZenReader->GetChunkCount();
	}

	void EnumerateChunks(TFunction<bool(FIoStoreTocChunkInfo&&)>&& Callback) const
	{
		if (ReaderType == EReaderType::Standard)
		{
			StandardReader->EnumerateChunks(MoveTemp(Callback));
		}
		else
		{
			ZenReader->EnumerateChunks(MoveTemp(Callback));
		}
	}

	TIoStatusOr<FIoStoreTocChunkInfo> GetChunkInfo(const FIoChunkId& ChunkId) const
	{
		return ReaderType == EReaderType::Standard
			? StandardReader->GetChunkInfo(ChunkId)
			: ZenReader->GetChunkInfo(ChunkId);
	}

	TIoStatusOr<FIoStoreTocChunkInfo> GetChunkInfo(const uint32 TocEntryIndex) const
	{
		return ReaderType == EReaderType::Standard
			? StandardReader->GetChunkInfo(TocEntryIndex)
			: ZenReader->GetChunkInfo(TocEntryIndex);
	}

	// Read method - delegates to the appropriate reader
	TIoStatusOr<FIoBuffer> Read(const FIoChunkId& Chunk, const FIoReadOptions& Options) const
	{
		return ReaderType == EReaderType::Standard
			? StandardReader->Read(Chunk, Options)
			: ZenReader->Read(Chunk, Options);
	}

	EReaderType GetReaderType() const { return ReaderType; }
	FIoStoreReader* GetStandardReader() const { return StandardReader.Get(); }
	FIoStoreReaderZenBuild* GetZenReader() const { return ZenReader.Get(); }

private:
	TUniquePtr<FIoStoreReader> StandardReader;
	TUniquePtr<FIoStoreReaderZenBuild> ZenReader;
	EReaderType ReaderType;
};
