// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoStoreReaderZenBuild.h"
#include "IoStoreConfigHelpers.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformFileManager.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "Dom/JsonObject.h"
#include "Misc/Compression.h"
#include "Misc/AES.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

DEFINE_LOG_CATEGORY(LogZenBuildReader);

/**
 * Background thread runnable for processing batched zen download requests
 */
class FZenDownloadRunnable : public FRunnable
{
public:
	FZenDownloadRunnable(FIoStoreReaderZenBuild* InReader)
		: Reader(InReader)
	{
	}

	virtual uint32 Run() override
	{
		if (Reader)
		{
			Reader->FetchThread();
		}
		return 0;
	}

private:
	FIoStoreReaderZenBuild* Reader;
};

FIoStoreReaderZenBuild::FIoStoreReaderZenBuild()
{
}

FIoStoreReaderZenBuild::~FIoStoreReaderZenBuild()
{
	// Signal shutdown to background thread
	bShuttingDown = true;

	// Wake the thread so it can exit
	if (WakeEvent)
	{
		WakeEvent->Trigger();
	}

	// Wait for background thread to complete
	if (FetchThreadHandle)
	{
		FetchThreadHandle->WaitForCompletion();
		delete FetchThreadHandle;
		FetchThreadHandle = nullptr;
	}

	// Delete the runnable (FRunnableThread does not own it)
	if (FetchRunnable)
	{
		delete FetchRunnable;
		FetchRunnable = nullptr;
	}

	// Return event to pool
	if (WakeEvent)
	{
		FPlatformProcess::ReturnSynchEventToPool(WakeEvent);
		WakeEvent = nullptr;
	}

	// Kill any remaining zen.exe processes
	{
		FScopeLock Lock(&InFlightMutex);
		for (FProcHandle& Handle : InFlightProcesses)
		{
			if (Handle.IsValid())
			{
				FPlatformProcess::TerminateProc(Handle);
				FPlatformProcess::CloseProc(Handle);
			}
		}
		InFlightProcesses.Empty();
	}

	UE_LOGF(LogZenBuildReader, Display, "Zen build reader shut down");
}

FIoStatus FIoStoreReaderZenBuild::Initialize(
	FStringView InContainerPath,
	const TMap<FGuid, FAES::FAESKey>& InDecryptionKeys,
	const FString& InZenExePath,
	const FString& InOidcExePath,
	const FString& InNamespace,
	const FString& InBucketId,
	const FString& InBuildId,
	const FString& InProxyUrl,
	const FString& InBaseDownloadDirectory,
	const FString& InPartCacheDirectory)
{
	ContainerPath = InContainerPath;
	ZenExePath = InZenExePath;
	OidcExePath = InOidcExePath;
	Namespace = InNamespace;
	BucketId = InBucketId;
	BuildId = InBuildId;
	ProxyUrl = InProxyUrl;
	BaseDownloadDirectory = InBaseDownloadDirectory;
	PartCacheDirectory = InPartCacheDirectory;

	// Create the part cache directory if it doesn't exist
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.DirectoryExists(*PartCacheDirectory))
	{
		if (!PlatformFile.CreateDirectoryTree(*PartCacheDirectory))
		{
			return FIoStatusBuilder(EIoErrorCode::InvalidParameter)
				<< TEXT("Failed to create PartCache directory: ") << *PartCacheDirectory;
		}
	}

	// Read the TOC file
	TStringBuilder<256> TocFilePath;
	TocFilePath.Append(InContainerPath);
	TocFilePath.Append(TEXT(".utoc"));

	TIoStatusOr<TUniquePtr<IIoStoreTocReader>> TocReaderResult = IIoStoreTocReader::ReadFromDisk(*TocFilePath, InDecryptionKeys);
	if (!TocReaderResult.IsOk())
	{
		return TocReaderResult.Status();
	}
	TocReader = TocReaderResult.ConsumeValueOrDie();

	// Initialize decryption key if container is encrypted
	const FIoStoreTocResourceView& TocResource = TocReader->GetTocResource();
	if (EnumHasAnyFlags(TocResource.Header.ContainerFlags, EIoContainerFlags::Encrypted))
	{
		const FAES::FAESKey* FindKey = InDecryptionKeys.Find(TocResource.Header.EncryptionKeyGuid);
		if (!FindKey)
		{
			return FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
				<< TEXT("Missing decryption key for IoStore container file '") << *TocFilePath << TEXT("'");
		}
		DecryptionKey = *FindKey;
	}

	UE_LOGF(LogZenBuildReader, Display, "Initialized zen build reader for container: %ls", *GetContainerName());
	UE_LOGF(LogZenBuildReader, Display, "  PartCache directory: %ls", *PartCacheDirectory);

	// Initialize async batching infrastructure
	WakeEvent = FPlatformProcess::GetSynchEventFromPool(false);
	bShuttingDown = false;

	// Start background fetch thread
	FetchRunnable = new FZenDownloadRunnable(this);
	FetchThreadHandle = FRunnableThread::Create(
		FetchRunnable,
		TEXT("ZenBuildDownload"),
		0,  // Default stack size
		TPri_Normal);

	if (!FetchThreadHandle)
	{
		UE_LOGF(LogZenBuildReader, Error, "Failed to create background fetch thread");
		delete FetchRunnable;  // Clean up on failure - FRunnableThread::Create() does not delete the runnable on failure
		FetchRunnable = nullptr;
		return FIoStatusBuilder(EIoErrorCode::Unknown) << TEXT("Failed to create background fetch thread");
	}

	UE_LOGF(LogZenBuildReader, Display, "  Async batching enabled (MaxConcurrentDownloads=%d)", MaxConcurrentDownloads);

	return FIoStatus::Ok;
}

FIoContainerId FIoStoreReaderZenBuild::GetContainerId() const
{
	return TocReader->GetTocResource().Header.ContainerId;
}

uint32 FIoStoreReaderZenBuild::GetVersion() const
{
	return TocReader->GetTocResource().Header.Version;
}

EIoContainerFlags FIoStoreReaderZenBuild::GetContainerFlags() const
{
	return TocReader->GetTocResource().Header.ContainerFlags;
}

FGuid FIoStoreReaderZenBuild::GetEncryptionKeyGuid() const
{
	return TocReader->GetTocResource().Header.EncryptionKeyGuid;
}

FString FIoStoreReaderZenBuild::GetContainerName() const
{
	return FPaths::GetBaseFilename(ContainerPath);
}

int32 FIoStoreReaderZenBuild::GetChunkCount() const
{
	return TocReader->GetTocResource().ChunkIds.Num();
}

void FIoStoreReaderZenBuild::EnumerateChunks(TFunction<bool(FIoStoreTocChunkInfo&&)>&& Callback) const
{
	const FIoStoreTocResourceView& TocResource = TocReader->GetTocResource();

	for (int32 ChunkIndex = 0; ChunkIndex < TocResource.ChunkIds.Num(); ++ChunkIndex)
	{
		FIoStoreTocChunkInfo ChunkInfo = TocReader->GetTocChunkInfo(ChunkIndex);
		if (!Callback(MoveTemp(ChunkInfo)))
		{
			break;
		}
	}
}

TIoStatusOr<FIoStoreTocChunkInfo> FIoStoreReaderZenBuild::GetChunkInfo(const FIoChunkId& ChunkId) const
{
	const FIoStoreTocResourceView& TocResource = TocReader->GetTocResource();

	// Find the index of the chunk ID
	int32 TocEntryIndex = INDEX_NONE;
	for (int32 i = 0; i < TocResource.ChunkIds.Num(); ++i)
	{
		if (TocResource.ChunkIds[i] == ChunkId)
		{
			TocEntryIndex = i;
			break;
		}
	}

	if (TocEntryIndex != INDEX_NONE)
	{
		return TocReader->GetTocChunkInfo(TocEntryIndex);
	}
	else
	{
		return FIoStatus(EIoErrorCode::UnknownChunkID,
			FString::Printf(TEXT("Unknown chunk ID '%s'"), *LexToString(ChunkId)));
	}
}

TIoStatusOr<FIoStoreTocChunkInfo> FIoStoreReaderZenBuild::GetChunkInfo(const uint32 TocEntryIndex) const
{
	const FIoStoreTocResourceView& TocResource = TocReader->GetTocResource();

	if (TocEntryIndex < uint32(TocResource.ChunkIds.Num()))
	{
		return TocReader->GetTocChunkInfo(TocEntryIndex);
	}
	else
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid TOC entry index"));
	}
}

void FIoStoreReaderZenBuild::ClearCache()
{
	FScopeLock Lock(&AvailabilityCacheMutex);

	int32 NumEntries = DownloadedChunks.Num();
	DownloadedChunks.Empty();

	if (NumEntries > 0)
	{
		UE_LOGF(LogZenBuildReader, Display, "Cleared download cache (%d cached chunks)", NumEntries);
	}
}

TIoStatusOr<FIoBuffer> FIoStoreReaderZenBuild::Read(const FIoChunkId& Chunk, const FIoReadOptions& Options) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ReadChunk);

	// Get chunk info from TOC
	const FIoStoreTocResourceView& TocResource = TocReader->GetTocResource();

	// Find the chunk index
	int32 ChunkIndex = INDEX_NONE;
	for (int32 i = 0; i < TocResource.ChunkIds.Num(); ++i)
	{
		if (TocResource.ChunkIds[i] == Chunk)
		{
			ChunkIndex = i;
			break;
		}
	}

	if (ChunkIndex == INDEX_NONE)
	{
		return FIoStatus(EIoErrorCode::UnknownChunkID, FString::Printf(TEXT("Unknown chunk ID '%s'"), *LexToString(Chunk)));
	}

	const FIoOffsetAndLength& OffsetAndLength = TocResource.ChunkOffsetLengths[ChunkIndex];
	uint64 RequestedOffset = Options.GetOffset();
	uint64 ResolvedOffset = OffsetAndLength.GetOffset() + RequestedOffset;
	uint64 ResolvedSize = 0;
	if (RequestedOffset <= OffsetAndLength.GetLength())
	{
		ResolvedSize = FMath::Min(Options.GetSize(), OffsetAndLength.GetLength() - RequestedOffset);
	}

	// Guard against division by zero if CompressionBlockSize is 0 (malformed or legacy TOC)
	// Default to 64KB compression block size to match FIoStoreEnvironment default
	const uint64 CompressionBlockSize = TocResource.Header.CompressionBlockSize > 0
		? TocResource.Header.CompressionBlockSize
		: 65536ULL;

	FIoBuffer UncompressedBuffer(ResolvedSize);
	if (ResolvedSize == 0)
	{
		return UncompressedBuffer;
	}

	// Calculate which compression blocks we need
	int32 FirstBlockIndex = int32(ResolvedOffset / CompressionBlockSize);
	int32 LastBlockIndex = int32((Align(ResolvedOffset + ResolvedSize, CompressionBlockSize) - 1) / CompressionBlockSize);

	// Validate block indices to prevent out-of-bounds access on malformed TOCs
	if (FirstBlockIndex < 0 || FirstBlockIndex >= TocResource.CompressionBlocks.Num() ||
		LastBlockIndex < 0 || LastBlockIndex >= TocResource.CompressionBlocks.Num())
	{
		return FIoStatus(EIoErrorCode::CorruptToc,
			FString::Printf(TEXT("Block index out of bounds: FirstBlock=%d LastBlock=%d NumBlocks=%d"),
				FirstBlockIndex, LastBlockIndex, TocResource.CompressionBlocks.Num()));
	}

	// Calculate the byte range we need to fetch from zen
	const FIoStoreTocCompressedBlockEntry& FirstBlock = TocResource.CompressionBlocks[FirstBlockIndex];
	const FIoStoreTocCompressedBlockEntry& LastBlock = TocResource.CompressionBlocks[LastBlockIndex];

	// Guard against division by zero if PartitionSize is 0 (older TOC format)
	// Default to 1GB partition size to match FIoStoreTocBlockMap behavior
	const uint64 PartitionSize = TocResource.Header.PartitionSize > 0
		? TocResource.Header.PartitionSize
		: (1024ULL * 1024ULL * 1024ULL);

	// Calculate the absolute start and end positions
	const uint64 ReadStartAbsolute = FirstBlock.GetOffset();
	const uint64 ReadEndAbsolute = LastBlock.GetOffset() + Align(LastBlock.GetCompressedSize(), FAES::AESBlockSize);
	const uint64 CompressedSize = ReadEndAbsolute - ReadStartAbsolute;

	// Determine partition range (compression blocks may span partitions for large chunks)
	const int32 FirstPartitionIndex = static_cast<int32>(FirstBlock.GetOffset() / PartitionSize);
	const int32 LastPartitionIndex = static_cast<int32>(LastBlock.GetOffset() / PartitionSize);

	TArray64<uint8> CompressedData;

	if (FirstPartitionIndex == LastPartitionIndex)
	{
		// Single partition read (common case - ~99.9% of chunks)
		const uint64 ReadStartOffset = ReadStartAbsolute % PartitionSize;
		FString CacheKey = FString::Printf(TEXT("%d:%llu:%llu"), FirstPartitionIndex, ReadStartOffset, CompressedSize);

		{
			FScopeLock Lock(&AvailabilityCacheMutex);
			if (TArray64<uint8>* CachedData = DownloadedChunks.Find(CacheKey))
			{
				CompressedData = *CachedData;
				UE_LOGF(LogZenBuildReader, Display, "Cache hit for %ls (%.2f KB)", *CacheKey, CompressedData.Num() / 1024.0);
				goto decompress_and_return;  // Skip download
			}
		}

		// Cache miss - fetch from zen asynchronously
		UE_LOGF(LogZenBuildReader, Display, "Cache miss for %ls - queuing async fetch", *CacheKey);
		if (!FetchRangeAsync(FirstPartitionIndex, ReadStartOffset, CompressedSize, CompressedData))
		{
			return FIoStatus(EIoErrorCode::ReadError, TEXT("Failed to fetch data from zen"));
		}
	}
	else
	{
		// Cross-partition read (rare but valid for large chunks near partition boundaries)
		UE_LOGF(LogZenBuildReader, Display, "Cross-partition read: chunk spans partitions %d-%d (%.2f KB total)",
			FirstPartitionIndex, LastPartitionIndex, CompressedSize / 1024.0);

		CompressedData.SetNum(CompressedSize);
		uint64 DestOffset = 0;

		// Fetch each partition's contribution and combine into CompressedData
		for (int32 PartIdx = FirstPartitionIndex; PartIdx <= LastPartitionIndex; ++PartIdx)
		{
			// Calculate byte range within this partition
			const uint64 PartitionStart = static_cast<uint64>(PartIdx) * PartitionSize;
			const uint64 PartitionEnd = PartitionStart + PartitionSize;

			const uint64 ReadStart = FMath::Max(ReadStartAbsolute, PartitionStart);
			const uint64 ReadEnd = FMath::Min(ReadEndAbsolute, PartitionEnd);
			const uint64 ReadLength = ReadEnd - ReadStart;
			const uint64 ReadOffset = ReadStart % PartitionSize;

			TArray64<uint8> PartitionData;
			FString CacheKey = FString::Printf(TEXT("%d:%llu:%llu"), PartIdx, ReadOffset, ReadLength);

			// Try cache first
			bool bFoundInCache = false;
			{
				FScopeLock Lock(&AvailabilityCacheMutex);
				if (TArray64<uint8>* CachedData = DownloadedChunks.Find(CacheKey))
				{
					PartitionData = *CachedData;
					bFoundInCache = true;
					UE_LOGF(LogZenBuildReader, Display, "  Partition %d: cache hit (%.2f KB)", PartIdx, ReadLength / 1024.0);
				}
			}

			// Fetch if not cached
			if (!bFoundInCache)
			{
				UE_LOGF(LogZenBuildReader, Display, "  Partition %d: fetching %.2f KB at offset %llu", PartIdx, ReadLength / 1024.0, ReadOffset);
				if (!FetchRangeAsync(PartIdx, ReadOffset, ReadLength, PartitionData))
				{
					return FIoStatus(EIoErrorCode::ReadError, FString::Printf(TEXT("Failed to fetch cross-partition data from partition %d"), PartIdx));
				}
			}

			// Copy this partition's data into the combined buffer
			FMemory::Memcpy(CompressedData.GetData() + DestOffset, PartitionData.GetData(), ReadLength);
			DestOffset += ReadLength;
		}

		UE_LOGF(LogZenBuildReader, Display, "Cross-partition read complete: combined %llu bytes from %d partitions",
			DestOffset, LastPartitionIndex - FirstPartitionIndex + 1);
	}

decompress_and_return:

	// Decompress blocks and copy to output buffer
	uint64 CompressedSourceOffset = 0;
	uint64 UncompressedDestinationOffset = 0;
	uint64 OffsetInBlock = ResolvedOffset % CompressionBlockSize;
	uint64 RemainingSize = ResolvedSize;

	for (int32 BlockIndex = FirstBlockIndex; BlockIndex <= LastBlockIndex; ++BlockIndex)
	{
		const FIoStoreTocCompressedBlockEntry& CompressionBlock = TocResource.CompressionBlocks[BlockIndex];
		const uint32 RawSize = Align(CompressionBlock.GetCompressedSize(), FAES::AESBlockSize);
		const uint32 UncompressedSize = CompressionBlock.GetUncompressedSize();
		FName CompressionMethod = TocResource.CompressionMethods[CompressionBlock.GetCompressionMethodIndex()];

		uint8* CompressedSource = CompressedData.GetData() + CompressedSourceOffset;
		uint8* UncompressedDestination = UncompressedBuffer.Data() + UncompressedDestinationOffset;

		// Calculate actual bytes to write to output buffer (accounts for partial blocks)
		uint64 BytesToWrite = FMath::Min<uint64>(UncompressedSize - OffsetInBlock, RemainingSize);

		// Decrypt if needed
		if (EnumHasAnyFlags(TocResource.Header.ContainerFlags, EIoContainerFlags::Encrypted))
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(Decrypt);
			FAES::DecryptData(CompressedSource, RawSize, DecryptionKey);
		}

		// Decompress or copy
		if (CompressionMethod.IsNone())
		{
			// No compression - just copy
			FMemory::Memcpy(UncompressedDestination, CompressedSource + OffsetInBlock, BytesToWrite);
		}
		else
		{
			// Decompress
			bool bUncompressed;
			if (OffsetInBlock || RemainingSize < UncompressedSize)
			{
				// Need to decompress to temp buffer and copy partial
				TArray<uint8> TempBuffer;
				TempBuffer.SetNumUninitialized(UncompressedSize);
				bUncompressed = FCompression::UncompressMemory(CompressionMethod, TempBuffer.GetData(), UncompressedSize, CompressedSource, CompressionBlock.GetCompressedSize());
				FMemory::Memcpy(UncompressedDestination, TempBuffer.GetData() + OffsetInBlock, BytesToWrite);
			}
			else
			{
				// Decompress directly to output
				bUncompressed = FCompression::UncompressMemory(CompressionMethod, UncompressedDestination, UncompressedSize, CompressedSource, CompressionBlock.GetCompressedSize());
			}

			if (!bUncompressed)
			{
				return FIoStatus(EIoErrorCode::CorruptToc, TEXT("Failed to uncompress block"));
			}
		}

		// Advance to next block - use actual bytes written to avoid out-of-bounds pointer arithmetic
		CompressedSourceOffset += RawSize;
		UncompressedDestinationOffset += BytesToWrite;
		RemainingSize -= BytesToWrite;
		OffsetInBlock = 0; // Only applies to first block
	}

	return UncompressedBuffer;
}

UE::Tasks::FTask FIoStoreReaderZenBuild::StartAsyncRead(
	int32 InPartitionIndex,
	int64 InPartitionOffset,
	int64 InReadAmount,
	uint8* OutBuffer,
	std::atomic_bool* OutSuccess) const
{
	return UE::Tasks::Launch(TEXT("FIoStoreReaderZenBuild_AsyncRead"),
		[this, InPartitionIndex, InPartitionOffset, InReadAmount, OutBuffer, OutSuccess]() mutable
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FIoStoreReaderZenBuild_AsyncRead);

			// Try cache first
			FString CacheKey = FString::Printf(TEXT("%d:%llu:%llu"), InPartitionIndex, (uint64)InPartitionOffset, (uint64)InReadAmount);
			bool bCacheHit = false;

			{
				FScopeLock Lock(&AvailabilityCacheMutex);
				if (TArray64<uint8>* CachedData = DownloadedChunks.Find(CacheKey))
				{
					FMemory::Memcpy(OutBuffer, CachedData->GetData(), InReadAmount);
					OutSuccess->store(true);
					bCacheHit = true;
				}
			}

			if (!bCacheHit)
			{
				// Cache miss - fetch asynchronously
				TArray64<uint8> Data;
				bool bSuccess = FetchRangeAsync(InPartitionIndex, (uint64)InPartitionOffset, (uint64)InReadAmount, Data);

				if (bSuccess && Data.Num() == InReadAmount)
				{
					// Copy data to output buffer
					FMemory::Memcpy(OutBuffer, Data.GetData(), InReadAmount);
					OutSuccess->store(true);
				}
				else
				{
					UE_LOGF(LogZenBuildReader, Error, "Failed to fetch range [%lld, %lld) from partition %d",
						InPartitionOffset, InPartitionOffset + InReadAmount, InPartitionIndex);
					OutSuccess->store(false);
				}
			}
		});
}

// ============================================================================
// Async Batching Infrastructure - Phase 1 Stubs
// ============================================================================

void FIoStoreReaderZenBuild::FetchThread()
{
	UE_LOGF(LogZenBuildReader, Display, "Background fetch thread started (batch window: 150ms, max concurrent: %d)", MaxConcurrentDownloads);

	const uint32 BatchWindowMs = 150;  // Proven optimal by FPartialDownloadCoordinator

	while (!bShuttingDown)
	{
		// Wait for request or batch window timeout
		WakeEvent->Wait(BatchWindowMs);

		if (bShuttingDown)
		{
			break;
		}

		// Atomically collect all pending requests
		TArray<TSharedPtr<FZenChunkFetchRequest>> Batch;
		{
			FScopeLock Lock(&RequestQueueMutex);
			if (PendingRequests.Num() > 0)
			{
				Batch = MoveTemp(PendingRequests);
				PendingRequests.Reset();
			}
		}

		if (Batch.Num() > 0)
		{
			UE_LOGF(LogZenBuildReader, Display, "Processing batch of %d requests", Batch.Num());

			// Wait for an available concurrency slot
			WaitForConcurrencySlot();

			// Execute batch asynchronously (Phase 3 will implement)
			ExecuteFetchBatchAsync(Batch);
		}
	}

	// Wait for all in-flight downloads before exiting
	UE_LOGF(LogZenBuildReader, Display, "Background fetch thread shutting down...");
	WaitForAllInFlightDownloads();

	UE_LOGF(LogZenBuildReader, Display, "Background fetch thread exited");
}

bool FIoStoreReaderZenBuild::FetchRangeAsync(int32 PartitionIndex, uint64 Offset, uint64 Length, TArray64<uint8>& OutData) const
{
	// Reject new requests if shutting down
	if (bShuttingDown)
	{
		UE_LOGF(LogZenBuildReader, Warning, "Fetch request rejected - reader shutting down");
		return false;
	}

	// Create and configure request
	TSharedPtr<FZenChunkFetchRequest> Request = MakeShared<FZenChunkFetchRequest>();
	Request->PartitionIndex = PartitionIndex;
	Request->Offset = Offset;
	Request->Length = Length;
	Request->CompletionEvent = FPlatformProcess::GetSynchEventFromPool(false);
	Request->OutDataPtr = &OutData;
	Request->bCompleted = false;
	Request->bFailed = false;

	// Queue the request and wake thread only if this is the first request
	bool bShouldWake = false;
	{
		FScopeLock Lock(&RequestQueueMutex);
		bShouldWake = (PendingRequests.Num() == 0);  // Wake only if queue was empty
		PendingRequests.Add(Request);
	}

	// Wake the background thread only for the first request in a batch
	// Subsequent requests will be collected during the 150ms batch window
	if (bShouldWake && WakeEvent)
	{
		WakeEvent->Trigger();
	}

	// Wait for completion (blocking this thread until batch completes)
	const uint32 TimeoutMs = 120000;  // 2 minutes - enough for batch + download
	bool bSignaled = Request->CompletionEvent->Wait(TimeoutMs);

	// Return event to pool
	FPlatformProcess::ReturnSynchEventToPool(Request->CompletionEvent);
	Request->CompletionEvent = nullptr;

	if (!bSignaled)
	{
		UE_LOGF(LogZenBuildReader, Error, "Fetch request timed out after %u ms", TimeoutMs);
		Request->bTimedOut = true;  // Invalidate OutDataPtr - stack will be unwound
		return false;
	}

	if (Request->bFailed)
	{
		UE_LOGF(LogZenBuildReader, Error, "Fetch request failed");
		return false;
	}

	return true;
}

void FIoStoreReaderZenBuild::WaitForConcurrencySlot() const
{
	// Phase 2: Will implement concurrency limiting
	while (!bShuttingDown)
	{
		PollInFlightProcesses();

		{
			FScopeLock Lock(&InFlightMutex);
			if (InFlightProcesses.Num() < MaxConcurrentDownloads)
			{
				return;  // Slot available
			}
		}

		FPlatformProcess::Sleep(0.1f);
	}
}

void FIoStoreReaderZenBuild::PollInFlightProcesses() const
{
	// Phase 2: Will implement process cleanup
	FScopeLock Lock(&InFlightMutex);

	for (int32 i = InFlightProcesses.Num() - 1; i >= 0; --i)
	{
		if (!FPlatformProcess::IsProcRunning(InFlightProcesses[i]))
		{
			FPlatformProcess::CloseProc(InFlightProcesses[i]);
			InFlightProcesses.RemoveAt(i);
		}
	}
}

void FIoStoreReaderZenBuild::WaitForAllInFlightDownloads() const
{
	// Phase 2: Will implement graceful shutdown wait
	UE_LOGF(LogZenBuildReader, Display, "Waiting for all in-flight downloads to complete...");

	while (true)
	{
		{
			FScopeLock Lock(&InFlightMutex);
			if (InFlightProcesses.Num() == 0)
			{
				break;
			}
		}

		FPlatformProcess::Sleep(0.1f);
		PollInFlightProcesses();
	}

	UE_LOGF(LogZenBuildReader, Display, "All in-flight downloads complete");
}

void FIoStoreReaderZenBuild::ExecuteFetchBatchAsync(TArray<TSharedPtr<FZenChunkFetchRequest>>& Batch) const
{
	// Group requests by partition to minimize zen.exe invocations
	// Each partition gets its own zen.exe call with batched requests
	TMap<int32, TArray<TSharedPtr<FZenChunkFetchRequest>>> PartitionGroups;

	for (const TSharedPtr<FZenChunkFetchRequest>& Request : Batch)
	{
		PartitionGroups.FindOrAdd(Request->PartitionIndex).Add(Request);
	}

	UE_LOGF(LogZenBuildReader, Display, "Batch split into %d partition group(s)", PartitionGroups.Num());

	// Launch zen.exe for each partition group
	for (const auto& Group : PartitionGroups)
	{
		int32 PartitionIndex = Group.Key;
		const TArray<TSharedPtr<FZenChunkFetchRequest>>& Requests = Group.Value;

		UE_LOGF(LogZenBuildReader, Display, "  Partition %d: %d request(s)", PartitionIndex, Requests.Num());

		LaunchZenCommandAsync(PartitionIndex, Requests);
	}
}

void FIoStoreReaderZenBuild::LaunchZenCommandAsync(int32 PartitionIndex, const TArray<TSharedPtr<FZenChunkFetchRequest>>& Requests) const
{
	// Create unique batch directory
	static std::atomic<int32> BatchCounter{0};
	int32 BatchId = BatchCounter.fetch_add(1);
	FString BatchDirectory = FPaths::Combine(PartCacheDirectory, FString::Printf(TEXT("batch_%d_part_%d"), BatchId, PartitionIndex));

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	// Clean up existing batch directory if it exists
	if (PlatformFile.DirectoryExists(*BatchDirectory))
	{
		PlatformFile.DeleteDirectoryRecursively(*BatchDirectory);
	}

	// Create batch directory
	if (!PlatformFile.CreateDirectoryTree(*BatchDirectory))
	{
		UE_LOGF(LogZenBuildReader, Error, "Failed to create batch directory: %ls", *BatchDirectory);
		SignalBatchFailure(Requests);
		return;
	}

	// Generate manifest for all requests
	FString ManifestJson = GenerateBatchManifest(PartitionIndex, Requests);
	if (ManifestJson.IsEmpty())
	{
		UE_LOGF(LogZenBuildReader, Error, "Failed to generate batch manifest");
		SignalBatchFailure(Requests);
		return;
	}

	// Write manifest (outside batch directory to avoid --clean wiping it)
	FString ManifestPath = FPaths::Combine(PartCacheDirectory, FString::Printf(TEXT("manifest_batch_%d_part_%d.json"), BatchId, PartitionIndex));
	if (!FFileHelper::SaveStringToFile(ManifestJson, *ManifestPath))
	{
		UE_LOGF(LogZenBuildReader, Error, "Failed to write batch manifest to: %ls", *ManifestPath);
		SignalBatchFailure(Requests);
		return;
	}

	// Build zen command with properly escaped arguments
	FString ZenFolderPath = FPaths::Combine(BatchDirectory, TEXT(".zen"));
	FString Command = FString::Printf(
		TEXT("builds download --namespace=\"%s\" --bucket=\"%s\" --build-id=\"%s\" --host=\"%s\" --local-path=\"%s\" --download-spec-path=\"%s\" --allow-partial-block-requests=true --enable-scavenge=false --zen-folder-path=\"%s\""),
		*IoStoreConfig::EscapeCommandLineArgument(Namespace),
		*IoStoreConfig::EscapeCommandLineArgument(BucketId),
		*IoStoreConfig::EscapeCommandLineArgument(BuildId),
		*IoStoreConfig::EscapeCommandLineArgument(ProxyUrl),
		*IoStoreConfig::EscapeCommandLineArgument(BatchDirectory),
		*IoStoreConfig::EscapeCommandLineArgument(ManifestPath),
		*IoStoreConfig::EscapeCommandLineArgument(ZenFolderPath));

	if (!OidcExePath.IsEmpty())
	{
		Command += FString::Printf(TEXT(" --oidctoken-exe-path=\"%s\""), *IoStoreConfig::EscapeCommandLineArgument(OidcExePath));
	}

	// Launch zen.exe process non-blocking
	void* ReadPipe = nullptr;
	void* WritePipe = nullptr;
	if (!FPlatformProcess::CreatePipe(ReadPipe, WritePipe))
	{
		UE_LOGF(LogZenBuildReader, Error, "Failed to create pipe for zen output");
		SignalBatchFailure(Requests);
		return;
	}

	FProcHandle ProcHandle = FPlatformProcess::CreateProc(
		*ZenExePath,
		*Command,
		false,	// bLaunchDetached
		true,	// bLaunchHidden
		true,	// bLaunchReallyHidden
		nullptr,	// OutProcessID
		0,		// PriorityModifier
		nullptr,	// WorkingDirectory
		WritePipe,	// PipeWriteChild
		ReadPipe	// PipeReadChild
	);

	FPlatformProcess::ClosePipe(0, WritePipe);  // Close write end immediately

	if (!ProcHandle.IsValid())
	{
		UE_LOGF(LogZenBuildReader, Error, "Failed to launch zen.exe: %ls", *ZenExePath);
		FPlatformProcess::ClosePipe(ReadPipe, 0);
		SignalBatchFailure(Requests);
		return;
	}

	// Add to in-flight tracking
	{
		FScopeLock Lock(&InFlightMutex);
		InFlightProcesses.Add(ProcHandle);
	}

	UE_LOGF(LogZenBuildReader, Display, "Launched zen.exe batch #%d for partition %d with %d requests", BatchId, PartitionIndex, Requests.Num());

	// Launch completion poller task (non-blocking)
	UE::Tasks::Launch(TEXT("ZenCompletionPoller"),
		[this, ProcHandle, Requests, BatchDirectory, ManifestPath, ReadPipe, BatchId, PartitionIndex]() {
			PollZenCompletion(ProcHandle, Requests, BatchDirectory, ManifestPath, ReadPipe, BatchId, PartitionIndex);
			FPlatformProcess::ClosePipe(ReadPipe, 0);
		});
}

void FIoStoreReaderZenBuild::PollZenCompletion(FProcHandle ProcHandle, TArray<TSharedPtr<FZenChunkFetchRequest>> Requests, FString BatchDirectory, FString ManifestPath, void* ReadPipe, int32 BatchId, int32 PartitionIndex) const
{
	const double TimeoutSeconds = 600.0;  // 10 minutes
	const double StartTime = FPlatformTime::Seconds();
	bool bTimedOut = false;
	FString ZenOutput;

	// Poll for completion with timeout
	while (FPlatformProcess::IsProcRunning(ProcHandle))
	{
		if (FPlatformTime::Seconds() - StartTime > TimeoutSeconds)
		{
			UE_LOGF(LogZenBuildReader, Error, "Zen batch #%d partition %d timed out after %.0f seconds", BatchId, PartitionIndex, TimeoutSeconds);
			FPlatformProcess::TerminateProc(ProcHandle);
			bTimedOut = true;
			break;
		}

		// Read zen.exe output to prevent pipe buffer from filling
		FString NewOutput = FPlatformProcess::ReadPipe(ReadPipe);
		if (!NewOutput.IsEmpty())
		{
			ZenOutput += NewOutput;

			// Limit output size to prevent OOM on very verbose commands
			const int32 MaxOutputSize = 50000;  // ~50KB
			if (ZenOutput.Len() > MaxOutputSize)
			{
				ZenOutput = ZenOutput.Right(MaxOutputSize);
			}

			// Log output for debugging (use Verbose to avoid spam in normal logs)
			UE_LOGF(LogZenBuildReader, Verbose, "Zen batch #%d partition %d: %ls", BatchId, PartitionIndex, *NewOutput.TrimStartAndEnd());
		}

		FPlatformProcess::Sleep(0.1f);  // Poll every 100ms
	}

	// Read any remaining output
	FString FinalOutput = FPlatformProcess::ReadPipe(ReadPipe);
	if (!FinalOutput.IsEmpty())
	{
		ZenOutput += FinalOutput;
		UE_LOGF(LogZenBuildReader, Verbose, "Zen batch #%d partition %d final: %ls", BatchId, PartitionIndex, *FinalOutput.TrimStartAndEnd());
	}

	// Get exit code
	int32 ReturnCode = -1;
	FPlatformProcess::GetProcReturnCode(ProcHandle, &ReturnCode);

	// Remove from in-flight tracking
	{
		FScopeLock Lock(&InFlightMutex);
		for (int32 i = InFlightProcesses.Num() - 1; i >= 0; --i)
		{
			if (InFlightProcesses[i].Get() == ProcHandle.Get())
			{
				InFlightProcesses.RemoveAt(i);
				break;
			}
		}
	}

	FPlatformProcess::CloseProc(ProcHandle);

	// Process result
	if (bTimedOut || ReturnCode != 0)
	{
		UE_LOGF(LogZenBuildReader, Error, "Zen batch #%d partition %d download failed (exit code: %d, timeout: %ls)",
			BatchId, PartitionIndex, ReturnCode, bTimedOut ? TEXT("yes") : TEXT("no"));

		// Log last portion of output for error diagnosis (last 2000 chars to see the error)
		if (ZenOutput.Len() > 0)
		{
			FString ErrorOutput = ZenOutput.Right(2000);
			UE_LOGF(LogZenBuildReader, Error, "Zen batch #%d partition %d output (last 2000 chars):\n%ls",
				BatchId, PartitionIndex, *ErrorOutput);
		}

		SignalBatchFailure(Requests);
	}
	else
	{
		UE_LOGF(LogZenBuildReader, Display, "Zen batch #%d partition %d download completed successfully (%d requests)",
			BatchId, PartitionIndex, Requests.Num());
		ProcessBatchSuccess(Requests, BatchDirectory);
	}

	// Clean up batch directory and manifest file
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (PlatformFile.DirectoryExists(*BatchDirectory))
	{
		PlatformFile.DeleteDirectoryRecursively(*BatchDirectory);
	}
	if (PlatformFile.FileExists(*ManifestPath))
	{
		PlatformFile.DeleteFile(*ManifestPath);
	}
}

void FIoStoreReaderZenBuild::ProcessBatchSuccess(const TArray<TSharedPtr<FZenChunkFetchRequest>>& Requests, const FString& BatchDirectory) const
{
	FString PartitionName = GetContainerName();
	if (Requests.Num() > 0 && Requests[0]->PartitionIndex > 0)
	{
		PartitionName += FString::Printf(TEXT("_s%d"), Requests[0]->PartitionIndex);
	}

	for (const TSharedPtr<FZenChunkFetchRequest>& Request : Requests)
	{
		// Construct the .part file path based on offset
		FString PartFilePath = FPaths::Combine(BatchDirectory, FString::Printf(TEXT("%s.part%llu.ucas"), *PartitionName, Request->Offset));

		// Read the .part file
		TArray64<uint8> FileData;
		if (!FFileHelper::LoadFileToArray(FileData, *PartFilePath))
		{
			UE_LOGF(LogZenBuildReader, Error, "Failed to read .part file: %ls", *PartFilePath);
			Request->bFailed = true;
			Request->bCompleted = true;
			if (Request->CompletionEvent)
			{
				Request->CompletionEvent->Trigger();
			}
			continue;
		}

		// Verify size matches - if not, fail the request to prevent out-of-bounds access during decompression
		if ((uint64)FileData.Num() != Request->Length)
		{
			UE_LOGF(LogZenBuildReader, Error, "Part file size mismatch: expected %llu, got %d - failing request to prevent memory corruption", Request->Length, FileData.Num());
			Request->bFailed = true;
			Request->bCompleted = true;
			if (Request->CompletionEvent)
			{
				Request->CompletionEvent->Trigger();
			}
			continue;
		}

		// Check if request timed out - if so, caller has given up and OutDataPtr is invalid
		if (Request->bTimedOut)
		{
			UE_LOGF(LogZenBuildReader, Warning, "Request completed after timeout - discarding result");
			// Still add to cache for future requests
			FString CacheKey = FString::Printf(TEXT("%d:%llu:%llu"), Request->PartitionIndex, Request->Offset, Request->Length);
			{
				FScopeLock Lock(&AvailabilityCacheMutex);
				DownloadedChunks.Add(CacheKey, MoveTemp(FileData));
			}
			// Don't signal completion event - caller already timed out
			continue;
		}

		// Copy to output buffer (safe - caller is still waiting)
		*Request->OutDataPtr = MoveTemp(FileData);

		// Add to cache
		FString CacheKey = FString::Printf(TEXT("%d:%llu:%llu"), Request->PartitionIndex, Request->Offset, Request->Length);
		{
			FScopeLock Lock(&AvailabilityCacheMutex);
			DownloadedChunks.Add(CacheKey, *Request->OutDataPtr);
		}

		// Signal completion
		Request->bCompleted = true;
		if (Request->CompletionEvent)
		{
			Request->CompletionEvent->Trigger();
		}

		UE_LOGF(LogZenBuildReader, Verbose, "Completed request: partition %d, offset %llu, length %llu",
			Request->PartitionIndex, Request->Offset, Request->Length);
	}
}

void FIoStoreReaderZenBuild::SignalBatchFailure(const TArray<TSharedPtr<FZenChunkFetchRequest>>& Requests) const
{
	UE_LOGF(LogZenBuildReader, Error, "Batch failed - marking %d requests as failed", Requests.Num());

	for (const TSharedPtr<FZenChunkFetchRequest>& Request : Requests)
	{
		Request->bFailed = true;
		Request->bCompleted = true;

		if (Request->CompletionEvent)
		{
			Request->CompletionEvent->Trigger();
		}
	}
}

FString FIoStoreReaderZenBuild::GenerateBatchManifest(int32 PartitionIndex, const TArray<TSharedPtr<FZenChunkFetchRequest>>& Requests) const
{
	// Calculate the relative path from BaseDownloadDirectory to the container .ucas file
	FString PartitionName = GetContainerName();
	if (PartitionIndex > 0)
	{
		PartitionName += FString::Printf(TEXT("_s%d"), PartitionIndex);
	}

	FString FullUcasPath = ContainerPath;
	if (PartitionIndex > 0)
	{
		FullUcasPath += FString::Printf(TEXT("_s%d"), PartitionIndex);
	}
	FullUcasPath += TEXT(".ucas");

	FString PartitionPath;
	if (!BaseDownloadDirectory.IsEmpty() && FullUcasPath.StartsWith(BaseDownloadDirectory))
	{
		// Calculate relative path
		PartitionPath = FullUcasPath.Mid(BaseDownloadDirectory.Len());
		// Remove leading slash/backslash
		if (PartitionPath.StartsWith(TEXT("/")) || PartitionPath.StartsWith(TEXT("\\")))
		{
			PartitionPath = PartitionPath.Mid(1);
		}
		// Convert backslashes to forward slashes for zen
		PartitionPath = PartitionPath.Replace(TEXT("\\"), TEXT("/"));
	}
	else
	{
		// Fallback to just the filename
		PartitionPath = PartitionName + TEXT(".ucas");
	}

	// Build files array - one entry per request, all pointing to same .ucas file
	TArray<TSharedPtr<FJsonValue>> FilesArray;
	for (const TSharedPtr<FZenChunkFetchRequest>& Request : Requests)
	{
		// Generate unique outputPath for this request
		FString OutputPath = FString::Printf(TEXT("%s.part%llu.ucas"), *PartitionName, Request->Offset);

		TSharedPtr<FJsonObject> FileObject = MakeShared<FJsonObject>();
		FileObject->SetStringField(TEXT("path"), PartitionPath);
		FileObject->SetNumberField(TEXT("offset"), Request->Offset);
		FileObject->SetNumberField(TEXT("length"), Request->Length);
		FileObject->SetStringField(TEXT("outputPath"), OutputPath);

		FilesArray.Add(MakeShared<FJsonValueObject>(FileObject));
	}

	// Build manifest structure
	TSharedPtr<FJsonObject> DefaultPartObject = MakeShared<FJsonObject>();
	DefaultPartObject->SetArrayField(TEXT("files"), FilesArray);

	TSharedPtr<FJsonObject> PartsObject = MakeShared<FJsonObject>();
	PartsObject->SetObjectField(TEXT("default"), DefaultPartObject);

	TSharedPtr<FJsonObject> RootObject = MakeShared<FJsonObject>();
	RootObject->SetObjectField(TEXT("parts"), PartsObject);

	// Serialize to string
	FString OutputString;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
	if (FJsonSerializer::Serialize(RootObject.ToSharedRef(), Writer))
	{
		return OutputString;
	}

	return FString();
}
