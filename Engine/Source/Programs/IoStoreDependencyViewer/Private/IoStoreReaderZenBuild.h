// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IO/IoStore.h"
#include "Tasks/Task.h"
#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "HAL/Event.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Templates/SharedPointer.h"
#include <atomic>

DECLARE_LOG_CATEGORY_EXTERN(LogZenBuildReader, Log, All);

// Forward declaration for friend access
class FZenDownloadRunnable;

/**
 * IoStore reader that fetches UCAS data on-demand from zen cloud builds
 * instead of reading from local disk files.
 */
class FIoStoreReaderZenBuild
{
	friend class FZenDownloadRunnable;

public:
	FIoStoreReaderZenBuild();
	~FIoStoreReaderZenBuild();

	/**
	 * Initialize the reader with zen connection parameters
	 * @param InContainerPath Container path WITHOUT the .utoc extension (e.g., "D:/Builds/pakchunk0-WindowsClient")
	 * @param InDecryptionKeys Decryption keys for encrypted containers
	 * @param InZenExePath Path to zen.exe
	 * @param InOidcExePath Path to OidcToken.exe
	 * @param InNamespace Cloud namespace
	 * @param InBucketId Cloud bucket
	 * @param InBuildId Cloud build ID
	 * @param InProxyUrl Optional proxy URL
	 * @param InBaseDownloadDirectory Base directory where the build was downloaded
	 * @param InPartCacheDirectory Directory for caching .part files (unique per container)
	 */
	FIoStatus Initialize(
		FStringView InContainerPath,
		const TMap<FGuid, FAES::FAESKey>& InDecryptionKeys,
		const FString& InZenExePath,
		const FString& InOidcExePath,
		const FString& InNamespace,
		const FString& InBucketId,
		const FString& InBuildId,
		const FString& InProxyUrl,
		const FString& InBaseDownloadDirectory,
		const FString& InPartCacheDirectory);

	// IoStore interface methods
	FIoContainerId GetContainerId() const;
	uint32 GetVersion() const;
	EIoContainerFlags GetContainerFlags() const;
	FGuid GetEncryptionKeyGuid() const;
	FString GetContainerName() const;
	int32 GetChunkCount() const;
	void EnumerateChunks(TFunction<bool(FIoStoreTocChunkInfo&&)>&& Callback) const;
	TIoStatusOr<FIoStoreTocChunkInfo> GetChunkInfo(const FIoChunkId& ChunkId) const;
	TIoStatusOr<FIoStoreTocChunkInfo> GetChunkInfo(const uint32 TocEntryIndex) const;

	/**
	 * Read a chunk from the container (synchronous).
	 * This will download the required ranges from zen and decompress/decrypt as needed.
	 */
	TIoStatusOr<FIoBuffer> Read(const FIoChunkId& Chunk, const FIoReadOptions& Options) const;

	/**
	 * Start an async read from the container.
	 * This will download the requested range from zen if not already cached,
	 * then copy the data into the output buffer.
	 */
	UE::Tasks::FTask StartAsyncRead(
		int32 InPartitionIndex,
		int64 InPartitionOffset,
		int64 InReadAmount,
		uint8* OutBuffer,
		std::atomic_bool* OutSuccess) const;

	/**
	 * Clear the downloaded chunks cache to free memory.
	 * Should be called after loading completes to release memory used during downloads.
	 */
	void ClearCache();

private:
	// Request structure for async batching
	struct FZenChunkFetchRequest
	{
		int32 PartitionIndex = 0;       // Which .ucas partition
		uint64 Offset = 0;              // Offset within partition
		uint64 Length = 0;              // Bytes to fetch
		FEvent* CompletionEvent = nullptr;        // Signal when complete (from pool)
		TArray64<uint8>* OutDataPtr = nullptr;    // Where to write fetched data (MAY BE INVALID if bTimedOut=true)
		std::atomic<bool> bCompleted{false};
		std::atomic<bool> bFailed{false};
		std::atomic<bool> bTimedOut{false};  // Set when caller gives up waiting - OutDataPtr becomes invalid
	};

	// Background thread methods
	void FetchThread();
	void WaitForConcurrencySlot() const;
	void PollInFlightProcesses() const;
	void WaitForAllInFlightDownloads() const;

	// Async fetch methods
	bool FetchRangeAsync(int32 PartitionIndex, uint64 Offset, uint64 Length, TArray64<uint8>& OutData) const;

	// Batch execution methods
	void ExecuteFetchBatchAsync(TArray<TSharedPtr<FZenChunkFetchRequest>>& Batch) const;
	void LaunchZenCommandAsync(int32 PartitionIndex, const TArray<TSharedPtr<FZenChunkFetchRequest>>& Requests) const;
	void PollZenCompletion(FProcHandle ProcHandle, TArray<TSharedPtr<FZenChunkFetchRequest>> Requests, FString BatchDirectory, FString ManifestPath, void* ReadPipe, int32 BatchId, int32 PartitionIndex) const;
	void ProcessBatchSuccess(const TArray<TSharedPtr<FZenChunkFetchRequest>>& Requests, const FString& BatchDirectory) const;
	void SignalBatchFailure(const TArray<TSharedPtr<FZenChunkFetchRequest>>& Requests) const;
	FString GenerateBatchManifest(int32 PartitionIndex, const TArray<TSharedPtr<FZenChunkFetchRequest>>& Requests) const;

private:
	// TOC reader for chunk metadata
	TUniquePtr<IIoStoreTocReader> TocReader;

	// Container base path (without .utoc extension)
	FString ContainerPath;

	// Zen connection parameters
	FString ZenExePath;
	FString OidcExePath;
	FString Namespace;
	FString BucketId;
	FString BuildId;
	FString ProxyUrl;

	// Base directory where the build was downloaded (for calculating relative paths)
	FString BaseDownloadDirectory;

	// Directory for caching .part files (unique per container)
	FString PartCacheDirectory;

	// Decryption key for encrypted containers
	FAES::FAESKey DecryptionKey;

	// Async batching infrastructure - three separate mutexes for fine-grained concurrency
	mutable FCriticalSection AvailabilityCacheMutex;  // Protects DownloadedChunks cache
	mutable FCriticalSection RequestQueueMutex;       // Protects PendingRequests queue
	mutable FCriticalSection InFlightMutex;           // Protects InFlightProcesses

	// Thread coordination
	FEvent* WakeEvent = nullptr;                      // Wake event from pool for batching
	std::atomic<bool> bShuttingDown{false};           // Shutdown coordination flag
	FRunnableThread* FetchThreadHandle = nullptr;     // Background fetch thread
	FRunnable* FetchRunnable = nullptr;               // Runnable object (must be deleted manually)

	// Data structures for async operation
	mutable TMap<FString, TArray64<uint8>> DownloadedChunks;  // Cache: "PartitionIdx:Offset:Length" -> data
	mutable TArray<TSharedPtr<FZenChunkFetchRequest>> PendingRequests;  // Request queue
	mutable TArray<FProcHandle> InFlightProcesses;    // Track concurrent zen.exe processes
	int32 MaxConcurrentDownloads = 6;                 // Tunable parallelism limit (4-8 recommended)
};
