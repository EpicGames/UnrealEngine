// Copyright Epic Games, Inc. All Rights Reserved.

#include "PartialDownloadPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "HAL/FileManager.h"
#include "HAL/Event.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Misc/CommandLine.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "IoStoreConfigHelpers.h"
#include "Dom/JsonObject.h"
#include "IO/IoStore.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"

DEFINE_LOG_CATEGORY_STATIC(LogPartialDownload, Log, All);

//-----------------------------------------------------------------------------
// FIoStoreTocBlockMap Implementation
//-----------------------------------------------------------------------------

bool FIoStoreTocBlockMap::Initialize(const FString& TocFilePath)
{
	// Load the TOC file
	FIoStoreTocResource TocResource;
	FIoStatus Status = FIoStoreTocResource::Read(*TocFilePath, EIoStoreTocReadOptions::Default, TocResource);

	if (!Status.IsOk())
	{
		UE_LOGF(LogPartialDownload, Error, "Failed to read TOC file: %ls", *TocFilePath);
		return false;
	}

	// Extract base name from path (e.g., "pakchunk0-WindowsClient" from "pakchunk0-WindowsClient.utoc")
	BaseName = FPaths::GetBaseFilename(TocFilePath);

	// Get partition size from header
	PartitionSize = TocResource.Header.PartitionSize;
	if (PartitionSize == 0)
	{
		// Default to 1GB if not specified (older format)
		PartitionSize = 1024ULL * 1024ULL * 1024ULL;
	}

	// Extract compression block information
	const int32 BlockCount = TocResource.CompressionBlocks.Num();
	Blocks.Reserve(BlockCount);

	for (int32 i = 0; i < BlockCount; ++i)
	{
		const FIoStoreTocCompressedBlockEntry& Entry = TocResource.CompressionBlocks[i];

		FBlockInfo Block;
		Block.Offset = Entry.GetOffset();
		Block.CompressedSize = Entry.GetCompressedSize();
		Block.UncompressedSize = Entry.GetUncompressedSize();
		Block.CompressionMethodIndex = Entry.GetCompressionMethodIndex();

		// Calculate which partition this block belongs to
		Block.PartitionIndex = static_cast<uint32>(Block.Offset / PartitionSize);

		Blocks.Add(Block);
	}

	UE_LOGF(LogPartialDownload, Display, "Loaded TOC: %ls (%d blocks, partition size: %llu)",
		*BaseName, BlockCount, PartitionSize);

	return true;
}

void FIoStoreTocBlockMap::GetBlocksForRange(uint64 FileOffset, uint64 Length, TArray<int32>& OutBlockIndices) const
{
	OutBlockIndices.Reset();

	if (Length == 0 || Blocks.Num() == 0)
	{
		return;
	}

	const uint64 EndOffset = FileOffset + Length;

	// Blocks are stored sequentially by offset, so we can binary search for the first relevant block
	// Then scan forward until we're past the requested range

	// Find first block that could contain our start offset
	int32 StartIndex = 0;
	int32 EndIndex = Blocks.Num() - 1;

	while (StartIndex < EndIndex)
	{
		int32 MidIndex = (StartIndex + EndIndex) / 2;
		const FBlockInfo& MidBlock = Blocks[MidIndex];

		// Block covers range [Offset, Offset + UncompressedSize)
		uint64 BlockStart = 0;
		for (int32 i = 0; i <= MidIndex; ++i)
		{
			if (i == MidIndex)
			{
				break;
			}
			BlockStart += Blocks[i].UncompressedSize;
		}

		if (BlockStart + MidBlock.UncompressedSize <= FileOffset)
		{
			StartIndex = MidIndex + 1;
		}
		else
		{
			EndIndex = MidIndex;
		}
	}

	// Scan from StartIndex and collect all overlapping blocks
	// Calculate the uncompressed offset at StartIndex
	uint64 CurrentUncompressedOffset = 0;
	for (int32 i = 0; i < StartIndex; ++i)
	{
		CurrentUncompressedOffset += Blocks[i].UncompressedSize;
	}

	// Now scan from StartIndex to find all overlapping blocks
	for (int32 i = StartIndex; i < Blocks.Num(); ++i)
	{
		const FBlockInfo& Block = Blocks[i];
		uint64 BlockStart = CurrentUncompressedOffset;
		uint64 BlockEnd = CurrentUncompressedOffset + Block.UncompressedSize;

		// Check if this block overlaps with our requested range
		if (BlockEnd > FileOffset && BlockStart < EndOffset)
		{
			OutBlockIndices.Add(i);
		}

		// If we've gone past the requested range, we can stop
		if (BlockStart >= EndOffset)
		{
			break;
		}

		CurrentUncompressedOffset += Block.UncompressedSize;
	}
}

const FBlockInfo* FIoStoreTocBlockMap::GetBlockInfo(int32 BlockIndex) const
{
	if (BlockIndex >= 0 && BlockIndex < Blocks.Num())
	{
		return &Blocks[BlockIndex];
	}
	return nullptr;
}

//-----------------------------------------------------------------------------
// FPartialDownloadCoordinator Implementation
//-----------------------------------------------------------------------------

class FBlockFetchRunnable : public FRunnable
{
public:
	FBlockFetchRunnable(FPartialDownloadCoordinator* InCoordinator)
		: Coordinator(InCoordinator)
	{
	}

	virtual uint32 Run() override
	{
		if (Coordinator)
		{
			Coordinator->FetchThread();
		}
		return 0;
	}

private:
	FPartialDownloadCoordinator* Coordinator;
};

FPartialDownloadCoordinator::FPartialDownloadCoordinator()
{
	WakeEvent = FPlatformProcess::GetSynchEventFromPool(false);
}

FPartialDownloadCoordinator::~FPartialDownloadCoordinator()
{
	Shutdown();

	if (WakeEvent)
	{
		FPlatformProcess::ReturnSynchEventToPool(WakeEvent);
		WakeEvent = nullptr;
	}
}

bool FPartialDownloadCoordinator::Initialize(
	const FString& InDownloadDirectory,
	const FString& InZenExePath,
	const FString& InOidcExePath,
	const FString& InNamespace,
	const FString& InBucketId,
	const FString& InBuildId,
	const FString& InProxyUrl)
{
	DownloadDirectory = InDownloadDirectory;
	ZenExePath = InZenExePath;
	OidcExePath = InOidcExePath;
	Namespace = InNamespace;
	BucketId = InBucketId;
	BuildId = InBuildId;
	ProxyUrl = InProxyUrl;

	// Start background fetch thread
	bShuttingDown = false;
	FetchRunnable = new FBlockFetchRunnable(this);
	FetchThreadHandle = FRunnableThread::Create(FetchRunnable, TEXT("PartialDownloadFetch"));

	if (!FetchThreadHandle)
	{
		UE_LOGF(LogPartialDownload, Error, "Failed to create partial download fetch thread");
		delete FetchRunnable;  // Clean up the runnable if thread creation failed
		FetchRunnable = nullptr;
		return false;
	}

	return true;
}

void FPartialDownloadCoordinator::RegisterBlockMap(const FString& UcasBasePath, TSharedPtr<FIoStoreTocBlockMap> BlockMap)
{
	FScopeLock Lock(&BlockMapsMutex);
	BlockMaps.Add(UcasBasePath, BlockMap);

	UE_LOGF(LogPartialDownload, Display, "Registered block map for: %ls", *UcasBasePath);
}

bool FPartialDownloadCoordinator::IsBlockAvailable(const FString& UcasFilePath, int32 BlockIndex) const
{
	FScopeLock Lock(&AvailabilityMutex);
	FString Key = FString::Printf(TEXT("%s:%d"), *UcasFilePath, BlockIndex);
	return AvailableBlocks.Contains(Key);
}

bool FPartialDownloadCoordinator::FetchBlocks(const FString& UcasFilePath, const TArray<int32>& BlockIndices)
{
	if (bShuttingDown)
	{
		UE_LOGF(LogPartialDownload, Warning, "Coordinator is shutting down, cannot fetch blocks");
		return false;
	}

	if (BlockIndices.Num() == 0)
	{
		return true;
	}

	// Check if all blocks are already available
	{
		FScopeLock Lock(&AvailabilityMutex);
		bool bAllAvailable = true;
		for (int32 BlockIndex : BlockIndices)
		{
			FString Key = FString::Printf(TEXT("%s:%d"), *UcasFilePath, BlockIndex);
			if (!AvailableBlocks.Contains(Key))
			{
				bAllAvailable = false;
				break;
			}
		}
		if (bAllAvailable)
		{
			return true;
		}
	}

	// Create fetch request
	TSharedPtr<FBlockFetchRequest> Request = MakeShared<FBlockFetchRequest>();
	Request->UcasFilePath = UcasFilePath;
	Request->BlockIndices = BlockIndices;
	Request->CompletionEvent = FPlatformProcess::GetSynchEventFromPool(false);

	// Add to queue
	{
		FScopeLock Lock(&RequestQueueMutex);
		PendingRequests.Add(Request);
	}

	// Wake the fetch thread
	if (WakeEvent)
	{
		WakeEvent->Trigger();
	}

	// Wait for completion (with timeout)
	const uint32 TimeoutMs = 120000; // 2 minutes
	Request->CompletionEvent->Wait(TimeoutMs);

	// Cleanup
	FPlatformProcess::ReturnSynchEventToPool(Request->CompletionEvent);
	Request->CompletionEvent = nullptr;

	return Request->bCompleted && !Request->bFailed;
}

bool FPartialDownloadCoordinator::EnsureRangeAvailable(const FString& UcasFilePath, uint64 Offset, uint64 Length)
{
	if (bShuttingDown)
	{
		UE_LOGF(LogPartialDownload, Warning, "Coordinator is shutting down, cannot fetch blocks");
		return false;
	}

	if (Length == 0)
	{
		return true;
	}

	// Extract base path (remove partition suffix and .ucas extension) and partition index
	FString BasePath = UcasFilePath;
	BasePath.ReplaceInline(TEXT(".ucas"), TEXT(""));

	// Extract partition index from suffix (_s1 = index 0, _s2 = index 1, etc.)
	int32 RequestedPartitionIndex = 0;
	int32 UnderscorePos = INDEX_NONE;
	if (BasePath.FindLastChar('_', UnderscorePos) && UnderscorePos > 0)
	{
		FString Suffix = BasePath.Mid(UnderscorePos);
		if (Suffix.StartsWith(TEXT("_s")) && Suffix.Len() > 2)
		{
			FString PartitionNumStr = Suffix.Mid(2);
			if (FCString::IsNumeric(*PartitionNumStr))
			{
				int32 PartitionSuffixNum = FCString::Atoi(*PartitionNumStr);

				// Partition index is 0-based: _s1 = 0, _s2 = 1, etc.
				// Note: _s0 is not a valid UE partition suffix (first partition has no suffix)
				if (PartitionSuffixNum < 1)
				{
					UE_LOGF(LogPartialDownload, Warning, "Invalid partition suffix '%ls' in path: %ls (expected _s1, _s2, etc.)", *Suffix, *UcasFilePath);
					return false;
				}

				RequestedPartitionIndex = PartitionSuffixNum - 1;
				BasePath = BasePath.Left(UnderscorePos);
			}
		}
	}

	// Find the block map
	TSharedPtr<FIoStoreTocBlockMap> BlockMap;
	{
		FScopeLock Lock(&BlockMapsMutex);
		BlockMap = BlockMaps.FindRef(BasePath);
	}

	if (!BlockMap.IsValid())
	{
		UE_LOGF(LogPartialDownload, Warning, "No block map found for: %ls (base: %ls)", *UcasFilePath, *BasePath);
		return false;
	}

	// Find blocks that match this partition and overlap with the requested range
	// IoStoreReader reads from partition files using compressed offsets (Block.Offset % PartitionSize)
	// So we need to find blocks where:
	//   1. Block.PartitionIndex == RequestedPartitionIndex
	//   2. (Block.Offset % PartitionSize) overlaps with [Offset, Offset + Length)
	TArray<int32> BlockIndices;
	uint64 PartitionSize = BlockMap->GetPartitionSize();
	uint64 EndOffset = Offset + Length;

	int32 BlockCount = BlockMap->GetBlockCount();
	for (int32 i = 0; i < BlockCount; ++i)
	{
		const FBlockInfo* Block = BlockMap->GetBlockInfo(i);
		if (!Block)
		{
			continue;
		}

		// Check if this block belongs to the requested partition
		if (Block->PartitionIndex != (uint32)RequestedPartitionIndex)
		{
			continue;
		}

		// Calculate this block's offset within its partition file
		uint64 BlockOffsetInPartition = (PartitionSize > 0) ? (Block->Offset % PartitionSize) : Block->Offset;
		uint64 BlockEndInPartition = BlockOffsetInPartition + Block->CompressedSize;

		// Check if this block overlaps with the requested range
		if (BlockEndInPartition > Offset && BlockOffsetInPartition < EndOffset)
		{
			BlockIndices.Add(i);
		}
	}

	// Filter to only blocks that aren't already available
	TArray<int32> MissingBlocks;
	{
		FScopeLock Lock(&AvailabilityMutex);
		for (int32 BlockIndex : BlockIndices)
		{
			FString Key = FString::Printf(TEXT("%s:%d"), *UcasFilePath, BlockIndex);
			if (!AvailableBlocks.Contains(Key))
			{
				MissingBlocks.Add(BlockIndex);
			}
		}
	}

	if (MissingBlocks.Num() == 0)
	{
		// All blocks already available
		return true;
	}

	// Fetch the missing blocks
	return FetchBlocks(UcasFilePath, MissingBlocks);
}

void FPartialDownloadCoordinator::Shutdown()
{
	bShuttingDown = true;

	// Wake the thread so it can exit
	if (WakeEvent)
	{
		WakeEvent->Trigger();
	}

	// Wait for thread to finish
	if (FetchThreadHandle)
	{
		FetchThreadHandle->WaitForCompletion();
		delete FetchThreadHandle;
		FetchThreadHandle = nullptr;
	}

	// Delete the runnable (FRunnableThread does not own it)
	if (FetchRunnable)
	{
		delete FetchRunnable;
		FetchRunnable = nullptr;
	}
}

void FPartialDownloadCoordinator::FetchThread()
{
	const uint32 BatchWindowMs = 150; // Collect requests for 150ms before processing

	while (!bShuttingDown)
	{
		// Wait for a request or timeout
		WakeEvent->Wait(BatchWindowMs);

		if (bShuttingDown)
		{
			break;
		}

		// Collect pending requests
		TArray<TSharedPtr<FBlockFetchRequest>> Batch;
		{
			FScopeLock Lock(&RequestQueueMutex);
			if (PendingRequests.Num() > 0)
			{
				Batch = PendingRequests;
				PendingRequests.Reset();
			}
		}

		if (Batch.Num() > 0)
		{
			UE_LOGF(LogPartialDownload, Display, "Processing batch of %d fetch requests", Batch.Num());

			// Execute the batch
			bool bSuccess = ExecuteFetchBatch(Batch);

			// Signal completion for all requests
			for (TSharedPtr<FBlockFetchRequest>& Request : Batch)
			{
				Request->bCompleted = true;
				Request->bFailed = !bSuccess;
				if (Request->CompletionEvent)
				{
					Request->CompletionEvent->Trigger();
				}
			}
		}
	}
}

bool FPartialDownloadCoordinator::ExecuteFetchBatch(const TArray<TSharedPtr<FBlockFetchRequest>>& Requests)
{
	if (Requests.Num() == 0)
	{
		return true;
	}

	// Count total blocks being fetched
	int32 TotalBlocks = 0;
	for (const TSharedPtr<FBlockFetchRequest>& Request : Requests)
	{
		TotalBlocks += Request->BlockIndices.Num();
	}

	UE_LOGF(LogPartialDownload, Display, "Fetching %d blocks across %d files", TotalBlocks, Requests.Num());

	// Generate manifest JSON
	FString ManifestPath = FPaths::Combine(DownloadDirectory, TEXT("partial_manifest.json"));
	FString ManifestJson = GenerateManifest(Requests);

	if (ManifestJson.IsEmpty())
	{
		UE_LOGF(LogPartialDownload, Error, "Failed to generate manifest - no valid blocks found");
		return false;
	}

	UE_LOGF(LogPartialDownload, Verbose, "Generated manifest (%d bytes):", ManifestJson.Len());
	UE_LOGF(LogPartialDownload, Verbose, "%ls", *ManifestJson); // Log full manifest for debugging

	if (!FFileHelper::SaveStringToFile(ManifestJson, *ManifestPath))
	{
		UE_LOGF(LogPartialDownload, Error, "Failed to write manifest to: %ls", *ManifestPath);
		return false;
	}

	UE_LOGF(LogPartialDownload, Display, "Manifest written to: %ls", *ManifestPath);

	// Build zen.exe command with proper argument escaping to prevent injection
	// Note: Removed --enable-scavenge to avoid file locking issues when zen tries to move files
	FString Command = FString::Printf(
		TEXT("builds download --namespace=\"%s\" --bucket=\"%s\" --build-id=\"%s\" --local-path=\"%s\" --download-spec-path=\"%s\" --allow-partial-block-requests=true"),
		*IoStoreConfig::EscapeCommandLineArgument(Namespace),
		*IoStoreConfig::EscapeCommandLineArgument(BucketId),
		*IoStoreConfig::EscapeCommandLineArgument(BuildId),
		*IoStoreConfig::EscapeCommandLineArgument(DownloadDirectory),
		*IoStoreConfig::EscapeCommandLineArgument(ManifestPath));

	if (!OidcExePath.IsEmpty())
	{
		Command += FString::Printf(TEXT(" --oidctoken-exe-path=\"%s\""), *IoStoreConfig::EscapeCommandLineArgument(OidcExePath));
	}

	if (!ProxyUrl.IsEmpty())
	{
		Command += FString::Printf(TEXT(" --host=\"%s\""), *IoStoreConfig::EscapeCommandLineArgument(ProxyUrl));
	}

	UE_LOGF(LogPartialDownload, Display, "Running zen command: %ls", *Command);

	// Retry logic with exponential backoff
	const int32 MaxAttempts = 3;
	const float BaseDelaySeconds = 1.0f;

	for (int32 Attempt = 1; Attempt <= MaxAttempts; ++Attempt)
	{
		if (Attempt > 1)
		{
			float DelaySeconds = BaseDelaySeconds * FMath::Pow(2.0f, Attempt - 2);
			UE_LOGF(LogPartialDownload, Warning, "Retrying download (attempt %d/%d) after %.1f seconds...",
				Attempt, MaxAttempts, DelaySeconds);
			FPlatformProcess::Sleep(DelaySeconds);
		}

		// Run zen.exe
		FString Output;
		int32 ExitCode = RunZenCommand(Command, Output);

		if (ExitCode == 0)
		{
			// Success! Mark blocks as available
			{
				FScopeLock Lock(&AvailabilityMutex);
				for (const TSharedPtr<FBlockFetchRequest>& Request : Requests)
				{
					for (int32 BlockIndex : Request->BlockIndices)
					{
						FString Key = FString::Printf(TEXT("%s:%d"), *Request->UcasFilePath, BlockIndex);
						AvailableBlocks.Add(Key);
					}
				}
			}

			UE_LOGF(LogPartialDownload, Display, "Successfully downloaded %d blocks", TotalBlocks);
			return true;
		}

		// Log the failure
		UE_LOGF(LogPartialDownload, Error, "Zen download failed with exit code %d (attempt %d/%d)",
			ExitCode, Attempt, MaxAttempts);

		// Log output if not too large
		if (Output.Len() < 1000)
		{
			UE_LOGF(LogPartialDownload, Error, "Zen output: %ls", *Output);
		}
		else
		{
			UE_LOGF(LogPartialDownload, Error, "Zen output (first 1000 chars): %ls", *Output.Left(1000));
		}
	}

	UE_LOGF(LogPartialDownload, Error, "Failed to download blocks after %d attempts", MaxAttempts);
	return false;
}

FString FPartialDownloadCoordinator::GenerateManifest(const TArray<TSharedPtr<FBlockFetchRequest>>& Requests)
{
	// Group requests by file and collect all block indices
	TMap<FString, TArray<int32>> FileToBlocks;

	for (const TSharedPtr<FBlockFetchRequest>& Request : Requests)
	{
		if (!Request.IsValid() || Request->UcasFilePath.IsEmpty())
		{
			UE_LOGF(LogPartialDownload, Warning, "Skipping invalid request");
			continue;
		}

		TArray<int32>& Blocks = FileToBlocks.FindOrAdd(Request->UcasFilePath);
		Blocks.Append(Request->BlockIndices);
	}

	// Build JSON structure
	TSharedPtr<FJsonObject> RootObject = MakeShared<FJsonObject>();
	TSharedPtr<FJsonObject> PartsObject = MakeShared<FJsonObject>();
	TSharedPtr<FJsonObject> DefaultPartObject = MakeShared<FJsonObject>();
	TArray<TSharedPtr<FJsonValue>> FilesArray;

	for (const TPair<FString, TArray<int32>>& Pair : FileToBlocks)
	{
		const FString& UcasFilePath = Pair.Key;
		const TArray<int32>& BlockIndices = Pair.Value;

		// Find the block map for this file
		FString BasePath = UcasFilePath;

		// Remove partition suffix (_s1, _s2, etc.) to get base path
		BasePath.ReplaceInline(TEXT(".ucas"), TEXT(""));
		int32 UnderscorePos = INDEX_NONE;
		if (BasePath.FindLastChar('_', UnderscorePos) && UnderscorePos > 0)
		{
			FString Suffix = BasePath.Mid(UnderscorePos);
			if (Suffix.StartsWith(TEXT("_s")) && FCString::IsNumeric(*Suffix.Mid(2)))
			{
				BasePath = BasePath.Left(UnderscorePos);
			}
		}

		TSharedPtr<FIoStoreTocBlockMap> BlockMap;
		{
			FScopeLock Lock(&BlockMapsMutex);
			BlockMap = BlockMaps.FindRef(BasePath);
		}

		if (!BlockMap.IsValid())
		{
			UE_LOGF(LogPartialDownload, Warning, "No block map found for: %ls (base: %ls)", *UcasFilePath, *BasePath);
			continue;
		}

		// Get relative path from download directory
		FString RelativePath = UcasFilePath;
		if (RelativePath.StartsWith(DownloadDirectory))
		{
			RelativePath = RelativePath.Mid(DownloadDirectory.Len());
			if (RelativePath.StartsWith(TEXT("/")) || RelativePath.StartsWith(TEXT("\\")))
			{
				RelativePath = RelativePath.Mid(1);
			}
		}

		// Convert backslashes to forward slashes for JSON
		RelativePath.ReplaceInline(TEXT("\\"), TEXT("/"));

		// For each block, create a file entry with offset and length
		int32 ValidBlocks = 0;
		for (int32 BlockIndex : BlockIndices)
		{
			const FBlockInfo* BlockInfo = BlockMap->GetBlockInfo(BlockIndex);
			if (!BlockInfo)
			{
				UE_LOGF(LogPartialDownload, Warning, "Invalid block index %d for file %ls", BlockIndex, *UcasFilePath);
				continue;
			}

			// Validate block info
			if (BlockInfo->CompressedSize == 0)
			{
				UE_LOGF(LogPartialDownload, Warning, "Block %d has zero size, skipping", BlockIndex);
				continue;
			}

			// Build the correct partition path
			// PartitionIndex is 0-based: 0 = _s1, 1 = _s2, etc.
			FString PartitionPath = RelativePath;
			if (BlockMap->GetPartitionSize() > 0)
			{
				// This file uses partitions - need to replace/add the partition suffix
				// First, remove .ucas extension if present
				if (PartitionPath.EndsWith(TEXT(".ucas")))
				{
					PartitionPath = PartitionPath.LeftChop(5);
				}

				// Remove any existing partition suffix (_s1, _s2, etc.) to avoid doubling
				// Partition suffixes are _s followed by digits
				int32 LastUnderscoreS = PartitionPath.Find(TEXT("_s"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
				if (LastUnderscoreS != INDEX_NONE)
				{
					// Check if this is followed by digits (to ensure it's a partition suffix)
					bool bIsPartitionSuffix = true;
					for (int32 i = LastUnderscoreS + 2; i < PartitionPath.Len(); ++i)
					{
						if (!FChar::IsDigit(PartitionPath[i]))
						{
							bIsPartitionSuffix = false;
							break;
						}
					}

					if (bIsPartitionSuffix && LastUnderscoreS + 2 < PartitionPath.Len())
					{
						// Strip the existing partition suffix
						PartitionPath = PartitionPath.Left(LastUnderscoreS);
					}
				}

				// Append the correct partition suffix: _s1, _s2, etc.
				PartitionPath += FString::Printf(TEXT("_s%d.ucas"), BlockInfo->PartitionIndex + 1);
			}

			// Calculate partition-relative offset
			uint64 PartitionOffset = BlockInfo->Offset % BlockMap->GetPartitionSize();

			// Generate .part file path using only offset: .part{offset}.ucas
			// At read time, we'll scan for .part files and use file size to determine length
			FString OutputPath = PartitionPath.LeftChop(5); // Remove ".ucas"
			OutputPath += FString::Printf(TEXT(".part%llu.ucas"), PartitionOffset);

			TSharedPtr<FJsonObject> FileObject = MakeShared<FJsonObject>();
			FileObject->SetStringField(TEXT("path"), PartitionPath);
			FileObject->SetNumberField(TEXT("offset"), PartitionOffset);
			FileObject->SetNumberField(TEXT("length"), BlockInfo->CompressedSize);
			FileObject->SetStringField(TEXT("outputPath"), OutputPath);

			FilesArray.Add(MakeShared<FJsonValueObject>(FileObject));
			ValidBlocks++;
		}

		UE_LOGF(LogPartialDownload, Display, "Added %d blocks for file: %ls", ValidBlocks, *RelativePath);
	}

	if (FilesArray.Num() == 0)
	{
		UE_LOGF(LogPartialDownload, Error, "No valid file entries in manifest");
		return FString();
	}

	DefaultPartObject->SetArrayField(TEXT("files"), FilesArray);
	PartsObject->SetObjectField(TEXT("default"), DefaultPartObject);
	RootObject->SetObjectField(TEXT("parts"), PartsObject);

	// Serialize to string
	FString OutputString;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
	if (!FJsonSerializer::Serialize(RootObject.ToSharedRef(), Writer))
	{
		UE_LOGF(LogPartialDownload, Error, "Failed to serialize JSON manifest");
		return FString();
	}

	UE_LOGF(LogPartialDownload, Display, "Generated manifest with %d file entries", FilesArray.Num());

	return OutputString;
}

int32 FPartialDownloadCoordinator::RunZenCommand(const FString& Command, FString& OutResult)
{
	// Launch zen.exe without pipes to avoid pipe inheritance issues
	// We don't need to capture output - just need the exit code
	FProcHandle ProcHandle = FPlatformProcess::CreateProc(
		*ZenExePath,
		*Command,
		false,	// bLaunchDetached
		false,	// bLaunchHidden - show window for debugging
		false,	// bLaunchReallyHidden
		nullptr,
		0,		// PriorityModifier
		nullptr,
		nullptr,	// No stdout pipe
		nullptr);	// No stderr pipe

	if (!ProcHandle.IsValid())
	{
		UE_LOGF(LogPartialDownload, Error, "Failed to launch zen.exe: %ls", *ZenExePath);
		return -1;
	}

	// Wait for completion with timeout
	const double TimeoutSeconds = 120.0; // 2 minutes
	double StartTime = FPlatformTime::Seconds();

	while (FPlatformProcess::IsProcRunning(ProcHandle))
	{
		// Check for timeout
		double ElapsedTime = FPlatformTime::Seconds() - StartTime;
		if (ElapsedTime > TimeoutSeconds)
		{
			UE_LOGF(LogPartialDownload, Error, "zen.exe operation timed out after %.1f seconds", ElapsedTime);
			FPlatformProcess::TerminateProc(ProcHandle, true);
			FPlatformProcess::CloseProc(ProcHandle);
			return -1;
		}

		FPlatformProcess::Sleep(0.1f);
	}

	// Get exit code
	int32 ExitCode = 0;
	FPlatformProcess::GetProcReturnCode(ProcHandle, &ExitCode);
	FPlatformProcess::CloseProc(ProcHandle);

	OutResult = TEXT("zen.exe completed");
	return ExitCode;
}

//-----------------------------------------------------------------------------
// FPartialDownloadFileHandle Implementation
//-----------------------------------------------------------------------------

FPartialDownloadFileHandle::FPartialDownloadFileHandle(
	IFileHandle* InRealHandle,
	const FString& InFilePath,
	TSharedPtr<FPartialDownloadCoordinator> InCoordinator,
	IPlatformFile* InPlatformFile)
	: RealHandle(InRealHandle)
	, FilePath(InFilePath)
	, Coordinator(InCoordinator)
	, CurrentPosition(0)
	, PlatformFile(InPlatformFile)
{
	// Close the initial handle immediately - we'll reopen lazily when needed
	// This prevents zen.exe from being blocked by open file handles
	if (RealHandle.IsValid())
	{
		RealHandle.Reset();
	}
}

FPartialDownloadFileHandle::~FPartialDownloadFileHandle()
{
	// Ensure file is closed
	FScopeLock Lock(&HandleMutex);
	if (RealHandle.IsValid())
	{
		RealHandle.Reset();
	}
}

int64 FPartialDownloadFileHandle::Tell()
{
	FScopeLock Lock(&HandleMutex);
	return CurrentPosition;
}

bool FPartialDownloadFileHandle::Seek(int64 NewPosition)
{
	FScopeLock Lock(&HandleMutex);
	CurrentPosition = NewPosition;
	if (RealHandle.IsValid())
	{
		return RealHandle->Seek(NewPosition);
	}
	return true;
}

bool FPartialDownloadFileHandle::SeekFromEnd(int64 NewPositionRelativeToEnd)
{
	FScopeLock Lock(&HandleMutex);
	if (RealHandle.IsValid())
	{
		int64 FileSize = RealHandle->Size();
		CurrentPosition = FileSize + NewPositionRelativeToEnd;
		return RealHandle->SeekFromEnd(NewPositionRelativeToEnd);
	}
	return false;
}

bool FPartialDownloadFileHandle::Read(uint8* Destination, int64 BytesToRead)
{
	FScopeLock Lock(&HandleMutex);

	// Validate parameters before casting to uint64
	if (BytesToRead < 0)
	{
		UE_LOGF(LogPartialDownload, Error, "Invalid negative BytesToRead: %lld", BytesToRead);
		return false;
	}
	if (CurrentPosition < 0)
	{
		UE_LOGF(LogPartialDownload, Error, "Invalid negative CurrentPosition: %lld", CurrentPosition);
		return false;
	}

	// Close any open handle before fetching blocks to avoid file locking issues
	if (RealHandle.IsValid())
	{
		RealHandle.Reset();
	}

	// Ensure blocks are available before reading
	if (!EnsureBlocksAvailable(CurrentPosition, BytesToRead))
	{
		FString SafeFilePath = FilePath;
		UE_LOGF(LogPartialDownload, Error, "Failed to fetch blocks for range [%lld, %lld) in file: %ls",
			CurrentPosition, CurrentPosition + BytesToRead, *SafeFilePath);
		return false;
	}

	// Scan for available .part files
	TArray<FPartialDownloadCoordinator::FPartFileInfo> PartFiles;
	if (Coordinator.IsValid())
	{
		Coordinator->GetAvailablePartFiles(FilePath, PartFiles);
	}

	// Find the .part file that contains this range
	FString PartFilePath;
	uint64 OffsetWithinPartFile = 0;
	uint64 ReadStart = (uint64)CurrentPosition;
	uint64 ReadEnd = ReadStart + (uint64)BytesToRead;

	for (const auto& PartFile : PartFiles)
	{
		uint64 PartEnd = PartFile.StartOffset + PartFile.Length;
		if (ReadStart >= PartFile.StartOffset && ReadEnd <= PartEnd)
		{
			// This .part file fully contains our read range
			PartFilePath = PartFile.FilePath;
			OffsetWithinPartFile = ReadStart - PartFile.StartOffset;
			break;
		}
	}

	if (PartFilePath.IsEmpty())
	{
		UE_LOGF(LogPartialDownload, Error, "No .part file found containing range [%lld, %lld) for file: %ls",
			ReadStart, ReadEnd, *FilePath);
		return false;
	}

	// Open the .part file
	TUniquePtr<IFileHandle> PartHandle(PlatformFile->OpenRead(*PartFilePath, false));
	if (!PartHandle.IsValid())
	{
		UE_LOGF(LogPartialDownload, Error, "Failed to open .part file: %ls", *PartFilePath);
		return false;
	}

	// Seek to the correct position within the .part file
	if (!PartHandle->Seek(OffsetWithinPartFile))
	{
		UE_LOGF(LogPartialDownload, Error, "Failed to seek to offset %llu in .part file: %ls",
			OffsetWithinPartFile, *PartFilePath);
		return false;
	}

	// Read from the .part file
	bool bSuccess = PartHandle->Read(Destination, BytesToRead);
	if (bSuccess)
	{
		CurrentPosition += BytesToRead;
	}

	return bSuccess;
}

bool FPartialDownloadFileHandle::ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset)
{
	FScopeLock Lock(&HandleMutex);

	// Validate parameters before casting to uint64
	if (BytesToRead < 0)
	{
		UE_LOGF(LogPartialDownload, Error, "Invalid negative BytesToRead: %lld", BytesToRead);
		return false;
	}
	if (Offset < 0)
	{
		UE_LOGF(LogPartialDownload, Error, "Invalid negative Offset: %lld", Offset);
		return false;
	}

	// Close any open handle before fetching blocks
	if (RealHandle.IsValid())
	{
		RealHandle.Reset();
	}

	// Ensure blocks are available
	if (!EnsureBlocksAvailable(Offset, BytesToRead))
	{
		FString SafeFilePath = FilePath;
		UE_LOGF(LogPartialDownload, Error, "Failed to fetch blocks for range [%lld, %lld) in file: %ls",
			Offset, Offset + BytesToRead, *SafeFilePath);
		return false;
	}

	// Scan for available .part files
	TArray<FPartialDownloadCoordinator::FPartFileInfo> PartFiles;
	if (Coordinator.IsValid())
	{
		Coordinator->GetAvailablePartFiles(FilePath, PartFiles);
	}

	// Find the .part file that contains this range
	FString PartFilePath;
	uint64 OffsetWithinPartFile = 0;
	uint64 ReadStart = (uint64)Offset;
	uint64 ReadEnd = ReadStart + (uint64)BytesToRead;

	for (const auto& PartFile : PartFiles)
	{
		uint64 PartEnd = PartFile.StartOffset + PartFile.Length;
		if (ReadStart >= PartFile.StartOffset && ReadEnd <= PartEnd)
		{
			// This .part file fully contains our read range
			PartFilePath = PartFile.FilePath;
			OffsetWithinPartFile = ReadStart - PartFile.StartOffset;
			break;
		}
	}

	if (PartFilePath.IsEmpty())
	{
		UE_LOGF(LogPartialDownload, Error, "No .part file found containing range [%lld, %lld) for file: %ls",
			ReadStart, ReadEnd, *FilePath);
		return false;
	}

	// Open the .part file
	TUniquePtr<IFileHandle> PartHandle(PlatformFile->OpenRead(*PartFilePath, false));
	if (!PartHandle.IsValid())
	{
		UE_LOGF(LogPartialDownload, Error, "Failed to open .part file: %ls", *PartFilePath);
		return false;
	}

	// Seek to the correct position within the .part file
	if (!PartHandle->Seek(OffsetWithinPartFile))
	{
		UE_LOGF(LogPartialDownload, Error, "Failed to seek to offset %llu in .part file: %ls",
			OffsetWithinPartFile, *PartFilePath);
		return false;
	}

	// Read from the .part file
	return PartHandle->Read(Destination, BytesToRead);
}

bool FPartialDownloadFileHandle::Write(const uint8* Source, int64 BytesToWrite)
{
	// Suppress unused parameter warnings - write operations not supported
	(void)Source;
	(void)BytesToWrite;

	// Partial download files are read-only - write operations not supported
	UE_LOGF(LogPartialDownload, Warning, "Write operation not supported on partial download file: %ls", *FilePath);
	return false;
}

bool FPartialDownloadFileHandle::Flush(const bool bFullFlush)
{
	FScopeLock Lock(&HandleMutex);
	if (RealHandle.IsValid())
	{
		return RealHandle->Flush(bFullFlush);
	}
	// No file handle to flush - return true (success) since there's nothing to write
	return true;
}

bool FPartialDownloadFileHandle::Truncate(int64 NewSize)
{
	// Suppress unused parameter warning - truncate operations not supported
	(void)NewSize;

	// Partial download files are read-only - truncate operations not supported
	UE_LOGF(LogPartialDownload, Warning, "Truncate operation not supported on partial download file: %ls", *FilePath);
	return false;
}

bool FPartialDownloadFileHandle::EnsureBlocksAvailable(int64 Offset, int64 Length)
{
	if (!Coordinator.IsValid())
	{
		UE_LOGF(LogPartialDownload, Warning, "Coordinator is invalid, cannot fetch blocks");
		return false;
	}

	// Check if coordinator is shutting down
	if (Coordinator->IsShuttingDown())
	{
		UE_LOGF(LogPartialDownload, Warning, "Coordinator is shutting down, cannot fetch blocks");
		return false;
	}

	// CRITICAL: Close the file handle before calling zen.exe to avoid file locking issues
	// zen.exe needs to write to the .ucas file, but we have it open for reading
	RealHandle.Reset();

	// Use the coordinator's helper method to ensure the range is available
	bool bSuccess = Coordinator->EnsureRangeAvailable(FilePath, static_cast<uint64>(Offset), static_cast<uint64>(Length));

	if (!bSuccess)
	{
		UE_LOGF(LogPartialDownload, Warning, "Failed to ensure blocks available for range [%lld, %lld)", Offset, Offset + Length);

		// Try to reopen the file even if fetch failed
		ReopenFile();
		return false;
	}

	// Reopen the file to see the newly downloaded data
	if (!ReopenFile())
	{
		UE_LOGF(LogPartialDownload, Error, "Failed to reopen file after block fetch: %ls", *FilePath);
		return false;
	}

	return true;
}

bool FPartialDownloadFileHandle::ReopenFile()
{
	if (!PlatformFile)
	{
		UE_LOGF(LogPartialDownload, Error, "No platform file available for reopening");
		return false;
	}

	// Save current position
	int64 SavedPosition = CurrentPosition;

	// Close the current handle
	RealHandle.Reset();

	// Small delay to ensure OS file buffers are flushed
	FPlatformProcess::Sleep(0.01f);

	// Reopen the file to see newly written data
	// Use the lower-level platform file directly to avoid going through the wrapper again
	IFileHandle* NewHandle = PlatformFile->OpenRead(*FilePath, false);
	if (!NewHandle)
	{
		UE_LOGF(LogPartialDownload, Error, "Failed to reopen file: %ls", *FilePath);
		return false;
	}

	// Restore position
	if (!NewHandle->Seek(SavedPosition))
	{
		UE_LOGF(LogPartialDownload, Warning, "Failed to restore file position after reopen");
	}

	RealHandle.Reset(NewHandle);

	UE_LOGF(LogPartialDownload, Display, "Successfully reopened file after block download: %ls", *FilePath);
	return true;
}

//-----------------------------------------------------------------------------
// FPartialDownloadPlatformFile Implementation
//-----------------------------------------------------------------------------

FPartialDownloadPlatformFile::FPartialDownloadPlatformFile()
	: LowerLevel(nullptr)
{
}

FPartialDownloadPlatformFile::~FPartialDownloadPlatformFile()
{
	if (Coordinator.IsValid())
	{
		Coordinator->Shutdown();
	}
}

bool FPartialDownloadPlatformFile::InitializePartialDownload(
	const FString& InDownloadDirectory,
	const FString& InZenExePath,
	const FString& InOidcExePath,
	const FString& InNamespace,
	const FString& InBucketId,
	const FString& InBuildId,
	const FString& InProxyUrl)
{
	DownloadDirectory = InDownloadDirectory;

	// Create coordinator
	Coordinator = MakeShared<FPartialDownloadCoordinator>();
	if (!Coordinator->Initialize(
		InDownloadDirectory,
		InZenExePath,
		InOidcExePath,
		InNamespace,
		InBucketId,
		InBuildId,
		InProxyUrl))
	{
		UE_LOGF(LogPartialDownload, Error, "Failed to initialize partial download coordinator");
		Coordinator.Reset();
		return false;
	}

	return true;
}

void FPartialDownloadPlatformFile::ParseTocFiles()
{
	if (!Coordinator.IsValid())
	{
		return;
	}

	// Find all .utoc files in the download directory
	TArray<FString> TocFiles;
	IFileManager::Get().FindFilesRecursive(TocFiles, *DownloadDirectory, TEXT("*.utoc"), true, false);

	UE_LOGF(LogPartialDownload, Display, "Found %d .utoc files to parse", TocFiles.Num());

	for (const FString& TocFile : TocFiles)
	{
		// Parse the TOC file
		TSharedPtr<FIoStoreTocBlockMap> BlockMap = MakeShared<FIoStoreTocBlockMap>();
		if (BlockMap->Initialize(TocFile))
		{
			// Register with coordinator
			// The base path is the TOC file path without the .utoc extension
			FString BasePath = TocFile.LeftChop(5); // Remove ".utoc"
			Coordinator->RegisterBlockMap(BasePath, BlockMap);
		}
	}
}

bool FPartialDownloadPlatformFile::Initialize(IPlatformFile* Inner, const TCHAR* CmdLine)
{
	LowerLevel = Inner;
	return LowerLevel != nullptr;
}

bool FPartialDownloadPlatformFile::ShouldInterceptFile(const TCHAR* Filename) const
{
	// Intercept .ucas files in our download directory
	// Use proper path prefix check (not substring matching) to avoid false positives
	FString FilePath(Filename);
	return !DownloadDirectory.IsEmpty() && FPaths::IsUnderDirectory(FilePath, DownloadDirectory) && FilePath.EndsWith(TEXT(".ucas"));
}

IFileHandle* FPartialDownloadPlatformFile::OpenRead(const TCHAR* Filename, bool bAllowWrite)
{
	if (!LowerLevel)
	{
		UE_LOGF(LogPartialDownload, Error, "No lower level platform file available");
		return nullptr;
	}

	if (ShouldInterceptFile(Filename))
	{
		if (!Coordinator.IsValid())
		{
			UE_LOGF(LogPartialDownload, Warning, "Coordinator not available for partial download - falling back to normal read");
			return LowerLevel->OpenRead(Filename, bAllowWrite);
		}

		// Open the underlying file (which may be sparse/partial)
		IFileHandle* RealHandle = LowerLevel->OpenRead(Filename, bAllowWrite);
		if (!RealHandle)
		{
			// File doesn't exist yet - we need to pre-fetch at least the first block
			// This allows zen.exe to create the file without conflicts
			UE_LOGF(LogPartialDownload, Display, "File doesn't exist yet, pre-fetching first block: %ls", Filename);

			// Fetch the first 64KB (typical block size)
			if (Coordinator->EnsureRangeAvailable(Filename, 0, 65536))
			{
				UE_LOGF(LogPartialDownload, Display, "Successfully pre-fetched first block");

				// Give Windows time to release file locks from zen.exe
				// Without this, immediately trying to open the file can fail
				FPlatformProcess::Sleep(0.2f);

				// Retry opening the file a few times with delays
				for (int32 RetryCount = 0; RetryCount < 5 && !RealHandle; ++RetryCount)
				{
					RealHandle = LowerLevel->OpenRead(Filename, bAllowWrite);
					if (!RealHandle && RetryCount < 4)
					{
						UE_LOGF(LogPartialDownload, Warning, "Failed to open file after pre-fetch, retrying... (attempt %d/5)", RetryCount + 1);
						FPlatformProcess::Sleep(0.1f);
					}
				}
			}
			else
			{
				UE_LOGF(LogPartialDownload, Warning, "Failed to pre-fetch first block for: %ls", Filename);
			}
		}

		if (RealHandle)
		{
			// Wrap in our custom handle
			return new FPartialDownloadFileHandle(RealHandle, Filename, Coordinator, LowerLevel);
		}
		else
		{
			UE_LOGF(LogPartialDownload, Error, "Failed to open file for partial download: %ls", Filename);
		}

		return RealHandle;
	}

	// Pass through to lower level
	return LowerLevel->OpenRead(Filename, bAllowWrite);
}

// Delegate all other methods to LowerLevel
bool FPartialDownloadPlatformFile::FileExists(const TCHAR* Filename)
{
	return LowerLevel ? LowerLevel->FileExists(Filename) : false;
}

int64 FPartialDownloadPlatformFile::FileSize(const TCHAR* Filename)
{
	return LowerLevel ? LowerLevel->FileSize(Filename) : -1;
}

bool FPartialDownloadPlatformFile::DeleteFile(const TCHAR* Filename)
{
	return LowerLevel ? LowerLevel->DeleteFile(Filename) : false;
}

bool FPartialDownloadPlatformFile::IsReadOnly(const TCHAR* Filename)
{
	return LowerLevel ? LowerLevel->IsReadOnly(Filename) : false;
}

bool FPartialDownloadPlatformFile::MoveFile(const TCHAR* To, const TCHAR* From)
{
	return LowerLevel ? LowerLevel->MoveFile(To, From) : false;
}

bool FPartialDownloadPlatformFile::SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue)
{
	return LowerLevel ? LowerLevel->SetReadOnly(Filename, bNewReadOnlyValue) : false;
}

FDateTime FPartialDownloadPlatformFile::GetTimeStamp(const TCHAR* Filename)
{
	return LowerLevel ? LowerLevel->GetTimeStamp(Filename) : FDateTime::MinValue();
}

void FPartialDownloadPlatformFile::SetTimeStamp(const TCHAR* Filename, FDateTime DateTime)
{
	if (LowerLevel)
	{
		LowerLevel->SetTimeStamp(Filename, DateTime);
	}
}

FDateTime FPartialDownloadPlatformFile::GetAccessTimeStamp(const TCHAR* Filename)
{
	return LowerLevel ? LowerLevel->GetAccessTimeStamp(Filename) : FDateTime::MinValue();
}

FString FPartialDownloadPlatformFile::GetFilenameOnDisk(const TCHAR* Filename)
{
	return LowerLevel ? LowerLevel->GetFilenameOnDisk(Filename) : TEXT("");
}

IFileHandle* FPartialDownloadPlatformFile::OpenWrite(const TCHAR* Filename, bool bAppend, bool bAllowRead)
{
	return LowerLevel ? LowerLevel->OpenWrite(Filename, bAppend, bAllowRead) : nullptr;
}

bool FPartialDownloadPlatformFile::DirectoryExists(const TCHAR* Directory)
{
	return LowerLevel ? LowerLevel->DirectoryExists(Directory) : false;
}

bool FPartialDownloadPlatformFile::CreateDirectory(const TCHAR* Directory)
{
	return LowerLevel ? LowerLevel->CreateDirectory(Directory) : false;
}

bool FPartialDownloadPlatformFile::DeleteDirectory(const TCHAR* Directory)
{
	return LowerLevel ? LowerLevel->DeleteDirectory(Directory) : false;
}

FFileStatData FPartialDownloadPlatformFile::GetStatData(const TCHAR* FilenameOrDirectory)
{
	return LowerLevel ? LowerLevel->GetStatData(FilenameOrDirectory) : FFileStatData();
}

bool FPartialDownloadPlatformFile::IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor)
{
	return LowerLevel ? LowerLevel->IterateDirectory(Directory, Visitor) : false;
}

bool FPartialDownloadPlatformFile::IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor)
{
	return LowerLevel ? LowerLevel->IterateDirectoryRecursively(Directory, Visitor) : false;
}

bool FPartialDownloadPlatformFile::IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor)
{
	return LowerLevel ? LowerLevel->IterateDirectoryStat(Directory, Visitor) : false;
}

bool FPartialDownloadPlatformFile::IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor)
{
	return LowerLevel ? LowerLevel->IterateDirectoryStatRecursively(Directory, Visitor) : false;
}

void FPartialDownloadCoordinator::GetAvailablePartFiles(const FString& OriginalPath, TArray<FPartFileInfo>& OutPartFiles) const
{
	OutPartFiles.Reset();

	// Get the base file path without extension
	FString Directory = FPaths::GetPath(OriginalPath);
	FString BaseFileName = FPaths::GetBaseFilename(OriginalPath); // e.g., "global_s1"
	FString Extension = FPaths::GetExtension(OriginalPath); // e.g., "ucas"

	// Scan for .part{offset}.ucas files
	FString SearchPattern = FString::Printf(TEXT("%s.part*.%s"), *BaseFileName, *Extension);

	TArray<FString> FoundFiles;
	IFileManager::Get().FindFiles(FoundFiles, *(Directory / SearchPattern), true, false);

	for (const FString& FileName : FoundFiles)
	{
		// Parse offset from the filename, use file size for length
		// Expected format: "global_s1.part1000000.ucas"
		// Format: {base}.part{offset}.{extension}
		FString PartPrefix = FString::Printf(TEXT("%s.part"), *BaseFileName);
		if (FileName.StartsWith(PartPrefix) && FileName.EndsWith(FString::Printf(TEXT(".%s"), *Extension)))
		{
			// Extract offset from "global_s1.part1000000.ucas"
			FString OffsetStr = FileName.Mid(PartPrefix.Len());
			int32 ExtensionPos = OffsetStr.Find(FString::Printf(TEXT(".%s"), *Extension));
			if (ExtensionPos != INDEX_NONE)
			{
				OffsetStr = OffsetStr.Left(ExtensionPos);

				uint64 Offset = FCString::Strtoui64(*OffsetStr, nullptr, 10);
				FString FullPath = Directory / FileName;
				int64 FileSize = IFileManager::Get().FileSize(*FullPath);

				if (FileSize > 0)
				{
					FPartFileInfo Info;
					Info.FilePath = FullPath;
					Info.StartOffset = Offset;
					Info.Length = (uint64)FileSize;
					OutPartFiles.Add(Info);

					UE_LOGF(LogPartialDownload, Verbose, "Found .part file: %ls (offset=%llu, length=%llu)",
						*FileName, Offset, (uint64)FileSize);
				}
			}
		}
	}

	// Sort by offset for easier range matching
	OutPartFiles.Sort([](const FPartFileInfo& A, const FPartFileInfo& B) {
		return A.StartOffset < B.StartOffset;
	});
}
