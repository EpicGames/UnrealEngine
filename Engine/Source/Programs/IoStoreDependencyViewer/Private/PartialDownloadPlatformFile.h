// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/CriticalSection.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Array.h"
#include "IO/IoStore.h"

// Forward declarations
class FPartialDownloadCoordinator;
class FPartialDownloadPlatformFile;
class FBlockFetchRunnable;
class FRunnable;
class FRunnableThread;

/**
 * Information about a single compression block from a .utoc file
 */
struct FBlockInfo
{
	uint64 Offset;				// Absolute offset in .ucas partition(s)
	uint32 CompressedSize;		// Size to read from disk
	uint32 UncompressedSize;	// Size after decompression
	uint32 PartitionIndex;		// Which .ucas partition file (0 for _s1, 1 for _s2, etc.)
	uint8 CompressionMethodIndex;
};

/**
 * Parses .utoc files and provides efficient byte-range to compression-block mapping
 */
class FIoStoreTocBlockMap
{
public:
	/**
	 * Load and parse a .utoc file
	 * @param TocFilePath Path to the .utoc file
	 * @return true if successfully parsed
	 */
	bool Initialize(const FString& TocFilePath);

	/**
	 * Given a byte range in the logical .ucas file, return which compression blocks are needed
	 * @param FileOffset Starting byte offset
	 * @param Length Number of bytes to read
	 * @param OutBlockIndices Indices of blocks that cover this range
	 */
	void GetBlocksForRange(uint64 FileOffset, uint64 Length, TArray<int32>& OutBlockIndices) const;

	/**
	 * Get information about a specific block
	 */
	const FBlockInfo* GetBlockInfo(int32 BlockIndex) const;

	/**
	 * Get the total number of compression blocks
	 */
	int32 GetBlockCount() const { return Blocks.Num(); }

	/**
	 * Get the partition size for this container (determines which .ucas file a block belongs to)
	 */
	uint64 GetPartitionSize() const { return PartitionSize; }

	/**
	 * Get the base filename (without extension or partition suffix)
	 */
	const FString& GetBaseName() const { return BaseName; }

private:
	TArray<FBlockInfo> Blocks;
	uint64 PartitionSize = 0;
	FString BaseName;
};

/**
 * Request for fetching compression blocks
 */
struct FBlockFetchRequest
{
	FString UcasFilePath;		// Full path to the .ucas file
	TArray<int32> BlockIndices;	// Which blocks to fetch
	FEvent* CompletionEvent = nullptr;	// Event to signal when fetch completes
	bool bCompleted = false;
	bool bFailed = false;
};

/**
 * Coordinates on-demand downloading of compression blocks via zen.exe
 */
class FPartialDownloadCoordinator
{
	friend class FBlockFetchRunnable;

public:
	FPartialDownloadCoordinator();
	~FPartialDownloadCoordinator();

	/**
	 * Initialize the coordinator
	 * @param InDownloadDirectory Where downloaded files are stored
	 * @param InZenExePath Path to zen.exe
	 * @param InOidcExePath Path to OidcToken.exe
	 * @param InNamespace Cloud namespace
	 * @param InBucketId Cloud bucket
	 * @param InBuildId Cloud build ID
	 * @param InProxyUrl Optional proxy URL
	 * @return true if initialization succeeded (including thread creation), false otherwise
	 */
	bool Initialize(
		const FString& InDownloadDirectory,
		const FString& InZenExePath,
		const FString& InOidcExePath,
		const FString& InNamespace,
		const FString& InBucketId,
		const FString& InBuildId,
		const FString& InProxyUrl);

	/**
	 * Register a TOC block map for a container
	 * @param UcasBasePath Base path to .ucas files (without _s1 suffix)
	 * @param BlockMap The parsed TOC block map
	 */
	void RegisterBlockMap(const FString& UcasBasePath, TSharedPtr<FIoStoreTocBlockMap> BlockMap);

	/**
	 * Check if a specific block is available locally
	 * @param UcasFilePath Path to the .ucas file
	 * @param BlockIndex Which block to check
	 * @return true if the block has been downloaded
	 */
	bool IsBlockAvailable(const FString& UcasFilePath, int32 BlockIndex) const;

	/**
	 * Request blocks to be fetched. Blocks until the blocks are available or fetch fails.
	 * @param UcasFilePath Path to the .ucas file
	 * @param BlockIndices Which blocks to fetch
	 * @return true if blocks were successfully fetched
	 */
	bool FetchBlocks(const FString& UcasFilePath, const TArray<int32>& BlockIndices);

	/**
	 * Calculate which blocks are needed for a byte range and fetch any that are missing
	 * @param UcasFilePath Path to the .ucas file
	 * @param Offset Starting byte offset in the file
	 * @param Length Number of bytes to read
	 * @return true if all needed blocks are available
	 */
	bool EnsureRangeAvailable(const FString& UcasFilePath, uint64 Offset, uint64 Length);

	/**
	 * Shutdown the coordinator and stop background tasks
	 */
	void Shutdown();

	/**
	 * Check if the coordinator is shutting down
	 */
	bool IsShuttingDown() const { return bShuttingDown; }

	/**
	 * Information about a downloaded .part file
	 */
	struct FPartFileInfo
	{
		FString FilePath;		// Full path to the .part file
		uint64 StartOffset;		// Starting offset in the original file
		uint64 Length;			// Length of data in this .part file
	};

	/**
	 * Scan for available .part files for a given base path
	 * @param OriginalPath The original .ucas file path being requested
	 * @param OutPartFiles Array of available .part files with their ranges
	 */
	void GetAvailablePartFiles(const FString& OriginalPath, TArray<FPartFileInfo>& OutPartFiles) const;

private:
	/**
	 * Background thread that batches and processes fetch requests
	 */
	void FetchThread();

	/**
	 * Execute a batch of fetch requests via zen.exe
	 */
	bool ExecuteFetchBatch(const TArray<TSharedPtr<FBlockFetchRequest>>& Requests);

	/**
	 * Generate JSON manifest for zen.exe partial download
	 */
	FString GenerateManifest(const TArray<TSharedPtr<FBlockFetchRequest>>& Requests);

	/**
	 * Run zen.exe with the given command
	 */
	int32 RunZenCommand(const FString& Command, FString& OutResult);

private:
	// Configuration
	FString DownloadDirectory;
	FString ZenExePath;
	FString OidcExePath;
	FString Namespace;
	FString BucketId;
	FString BuildId;
	FString ProxyUrl;

	// Block maps for each container (keyed by base .ucas path)
	mutable FCriticalSection BlockMapsMutex;
	TMap<FString, TSharedPtr<FIoStoreTocBlockMap>> BlockMaps;

	// Availability tracking (keyed by "ucasfilepath:blockindex")
	mutable FCriticalSection AvailabilityMutex;
	TSet<FString> AvailableBlocks;

	// Fetch request queue
	mutable FCriticalSection RequestQueueMutex;
	TArray<TSharedPtr<FBlockFetchRequest>> PendingRequests;

	// Background fetch thread
	FRunnableThread* FetchThreadHandle = nullptr;
	FRunnable* FetchRunnable = nullptr;               // Runnable object (must be deleted manually)
	std::atomic<bool> bShuttingDown{false};
	FEvent* WakeEvent = nullptr;
};

/**
 * Custom file handle that fetches compression blocks on-demand
 */
class FPartialDownloadFileHandle : public IFileHandle
{
public:
	FPartialDownloadFileHandle(
		IFileHandle* InRealHandle,
		const FString& InFilePath,
		TSharedPtr<FPartialDownloadCoordinator> InCoordinator,
		IPlatformFile* InPlatformFile);

	virtual ~FPartialDownloadFileHandle();

	// IFileHandle interface
	virtual int64 Tell() override;
	virtual bool Seek(int64 NewPosition) override;
	virtual bool SeekFromEnd(int64 NewPositionRelativeToEnd) override;
	virtual bool Read(uint8* Destination, int64 BytesToRead) override;
	virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override;
	virtual bool Write(const uint8* Source, int64 BytesToWrite) override;
	virtual bool Flush(const bool bFullFlush) override;
	virtual bool Truncate(int64 NewSize) override;

private:
	/**
	 * Ensure the blocks covering the given range are available
	 */
	bool EnsureBlocksAvailable(int64 Offset, int64 Length);

	/**
	 * Reopen the file handle to see newly written data
	 */
	bool ReopenFile();

private:
	TUniquePtr<IFileHandle> RealHandle;
	FString FilePath;
	TSharedPtr<FPartialDownloadCoordinator> Coordinator;
	int64 CurrentPosition = 0;
	IPlatformFile* PlatformFile = nullptr;
	FCriticalSection HandleMutex;  // Protects all operations on RealHandle
};

/**
 * Custom IPlatformFile wrapper that intercepts .ucas file opens and provides on-demand fetching
 */
class FPartialDownloadPlatformFile : public IPlatformFile
{
public:
	static constexpr const TCHAR* GetTypeName()
	{
		return TEXT("PartialDownloadFile");
	}

	FPartialDownloadPlatformFile();
	virtual ~FPartialDownloadPlatformFile();

	/**
	 * Initialize the platform file wrapper
	 * @param InDownloadDirectory Where .utoc and partial .ucas files are stored
	 * @param InZenExePath Path to zen.exe
	 * @param InOidcExePath Path to OidcToken.exe
	 * @param InNamespace Cloud namespace
	 * @param InBucketId Cloud bucket
	 * @param InBuildId Cloud build ID
	 * @param InProxyUrl Optional proxy URL
	 * @return true if initialization succeeded
	 */
	bool InitializePartialDownload(
		const FString& InDownloadDirectory,
		const FString& InZenExePath,
		const FString& InOidcExePath,
		const FString& InNamespace,
		const FString& InBucketId,
		const FString& InBuildId,
		const FString& InProxyUrl);

	/**
	 * Parse all .utoc files in the download directory and build block maps
	 */
	void ParseTocFiles();

	/**
	 * Get the coordinator (for testing/debugging)
	 */
	TSharedPtr<FPartialDownloadCoordinator> GetCoordinator() const { return Coordinator; }

	// IPlatformFile interface
	virtual bool ShouldBeUsed(IPlatformFile* Inner, const TCHAR* CmdLine) const override { return false; }
	virtual bool Initialize(IPlatformFile* Inner, const TCHAR* CmdLine) override;
	virtual void InitializeAfterSetActive() override {}
	virtual IPlatformFile* GetLowerLevel() override { return LowerLevel; }
	virtual void SetLowerLevel(IPlatformFile* NewLowerLevel) override { LowerLevel = NewLowerLevel; }
	virtual const TCHAR* GetName() const override { return GetTypeName(); }

	virtual bool FileExists(const TCHAR* Filename) override;
	virtual int64 FileSize(const TCHAR* Filename) override;
	virtual bool DeleteFile(const TCHAR* Filename) override;
	virtual bool IsReadOnly(const TCHAR* Filename) override;
	virtual bool MoveFile(const TCHAR* To, const TCHAR* From) override;
	virtual bool SetReadOnly(const TCHAR* Filename, bool bNewReadOnlyValue) override;
	virtual FDateTime GetTimeStamp(const TCHAR* Filename) override;
	virtual void SetTimeStamp(const TCHAR* Filename, FDateTime DateTime) override;
	virtual FDateTime GetAccessTimeStamp(const TCHAR* Filename) override;
	virtual FString GetFilenameOnDisk(const TCHAR* Filename) override;

	virtual IFileHandle* OpenRead(const TCHAR* Filename, bool bAllowWrite = false) override;
	virtual IFileHandle* OpenWrite(const TCHAR* Filename, bool bAppend = false, bool bAllowRead = false) override;
	virtual bool DirectoryExists(const TCHAR* Directory) override;
	virtual bool CreateDirectory(const TCHAR* Directory) override;
	virtual bool DeleteDirectory(const TCHAR* Directory) override;
	virtual FFileStatData GetStatData(const TCHAR* FilenameOrDirectory) override;

	virtual bool IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;
	virtual bool IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) override;
	virtual bool IterateDirectoryStat(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override;
	virtual bool IterateDirectoryStatRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryStatVisitor& Visitor) override;

private:
	/**
	 * Check if this is a .ucas file that should be intercepted
	 */
	bool ShouldInterceptFile(const TCHAR* Filename) const;

private:
	IPlatformFile* LowerLevel = nullptr;
	TSharedPtr<FPartialDownloadCoordinator> Coordinator;
	FString DownloadDirectory;
};
