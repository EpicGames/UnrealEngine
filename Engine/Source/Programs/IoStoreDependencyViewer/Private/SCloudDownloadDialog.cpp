// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCloudDownloadDialog.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "Serialization/JsonSerializer.h"
#include "DesktopPlatformModule.h"
#include "PartialDownloadPlatformFile.h"
#include "IoStoreConfigHelpers.h"
#include "Internationalization/Regex.h"

void SCloudDownloadDialog::Construct(const FArguments& InArgs, TSharedPtr<SWindow> InParentWindow)
{
	ParentWindow = InParentWindow;

	// Initialize build types
	BuildTypeList.Add(MakeShared<FString>(TEXT("staged-build")));
	BuildTypeList.Add(MakeShared<FString>(TEXT("packaged-build")));
	SelectedBuildType = BuildTypeList[0];

	// Set default download directory (convert to absolute path)
	DownloadDirectory = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectDir(), TEXT("CookedBuild"), TEXT("IoStoreDependencyViewer")));

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(16.0f)
		[
			SNew(SVerticalBox)

			// Title
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 0, 0, 16)
			[
				SNew(STextBlock)
				.Text(FText::FromString(TEXT("Download Build from Cloud Storage")))
				.Font(FCoreStyle::GetDefaultFontStyle("Bold", 12))
			]

			// Namespace selection
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 0, 0, 8)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(0.3f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Namespace:")))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(0.6f)
				[
					SAssignNew(NamespaceComboBox, SComboBox<TSharedPtr<FString>>)
					.OptionsSource(&NamespaceList)
					.OnGenerateWidget(this, &SCloudDownloadDialog::OnGenerateNamespaceWidget)
					.OnSelectionChanged(this, &SCloudDownloadDialog::OnNamespaceSelectionChanged)
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &SCloudDownloadDialog::GetCurrentNamespaceText)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8, 0, 0, 0)
				[
					SNew(SButton)
					.Text(FText::FromString(TEXT("Refresh")))
					.OnClicked(this, &SCloudDownloadDialog::OnRefreshNamespacesClicked)
				]
			]

			// Build type selection
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 0, 0, 8)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(0.3f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Build Type:")))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(0.7f)
				[
					SAssignNew(BuildTypeComboBox, SComboBox<TSharedPtr<FString>>)
					.OptionsSource(&BuildTypeList)
					.OnGenerateWidget(this, &SCloudDownloadDialog::OnGenerateBuildTypeWidget)
					.OnSelectionChanged(this, &SCloudDownloadDialog::OnBuildTypeSelectionChanged)
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &SCloudDownloadDialog::GetCurrentBuildTypeText)
					]
				]
			]

			// Stream selection
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 0, 0, 8)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(0.3f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Stream:")))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(0.7f)
				[
					SAssignNew(StreamComboBox, SComboBox<TSharedPtr<FString>>)
					.OptionsSource(&StreamList)
					.OnGenerateWidget(this, &SCloudDownloadDialog::OnGenerateStreamWidget)
					.OnSelectionChanged(this, &SCloudDownloadDialog::OnStreamSelectionChanged)
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &SCloudDownloadDialog::GetCurrentStreamText)
					]
				]
			]

			// Platform selection
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 0, 0, 8)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(0.3f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Platform:")))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(0.7f)
				[
					SAssignNew(PlatformComboBox, SComboBox<TSharedPtr<FString>>)
					.OptionsSource(&PlatformList)
					.OnGenerateWidget(this, &SCloudDownloadDialog::OnGeneratePlatformWidget)
					.OnSelectionChanged(this, &SCloudDownloadDialog::OnPlatformSelectionChanged)
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &SCloudDownloadDialog::GetCurrentPlatformText)
					]
				]
			]

			// Query builds button
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 0, 0, 16)
			[
				SNew(SButton)
				.Text(FText::FromString(TEXT("Query Available Builds")))
				.HAlign(HAlign_Center)
				.OnClicked(this, &SCloudDownloadDialog::OnQueryBuildsClicked)
			]

			// Build selection
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 0, 0, 8)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(0.3f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Build:")))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(0.45f)
				[
					SAssignNew(BuildComboBox, SComboBox<TSharedPtr<FCloudBuild>>)
					.OptionsSource(&FilteredBuildList)
					.OnGenerateWidget(this, &SCloudDownloadDialog::OnGenerateBuildWidget)
					.OnSelectionChanged(this, &SCloudDownloadDialog::OnBuildSelectionChanged)
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &SCloudDownloadDialog::GetCurrentBuildText)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8, 0, 0, 0)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Filter:")))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(0.25f)
				.Padding(4, 0, 0, 0)
				[
					SAssignNew(BuildFilterTextBox, SEditableTextBox)
					.HintText(FText::FromString(TEXT("Regex filter...")))
					.OnTextChanged(this, &SCloudDownloadDialog::OnBuildFilterTextChanged)
				]
			]

			// Download directory
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 0, 0, 16)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(0.3f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT("Download To:")))
				]

				+ SHorizontalBox::Slot()
				.FillWidth(0.6f)
				[
					SAssignNew(DownloadDirectoryTextBox, SEditableTextBox)
					.Text(FText::FromString(DownloadDirectory))
					.OnTextChanged(this, &SCloudDownloadDialog::OnDownloadDirectoryChanged)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(8, 0, 0, 0)
				[
					SNew(SButton)
					.Text(FText::FromString(TEXT("Browse...")))
					.OnClicked(this, &SCloudDownloadDialog::OnBrowseDirectoryClicked)
				]
			]

			// Partial downloads checkbox
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 8, 0, 0)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.FillWidth(0.3f)
				[
					SNew(STextBlock)
					.Text(FText::GetEmpty())
				]

				+ SHorizontalBox::Slot()
				.FillWidth(0.7f)
				[
					SAssignNew(UsePartialDownloadsCheckBox, SCheckBox)
					.Content()
					[
						SNew(STextBlock)
						.Text(FText::FromString(TEXT("Use partial downloads (experimental)")))
					]
					.ToolTipText(FText::FromString(TEXT("Skip .ucas file download; content will be fetched on-demand from the cloud as needed, reducing bandwidth usage")))
				]
			]

			// Status text
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 16, 0, 16)
			[
				SAssignNew(StatusTextBlock, STextBlock)
				.Text(FText::FromString(TEXT("Ready")))
				.ColorAndOpacity(FLinearColor(0.7f, 0.7f, 0.7f))
			]

			// Buttons
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 8, 0)
				[
					SNew(SButton)
					.Text(FText::FromString(TEXT("Download")))
					.OnClicked(this, &SCloudDownloadDialog::OnDownloadClicked)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Text(FText::FromString(TEXT("Cancel")))
					.OnClicked(this, &SCloudDownloadDialog::OnCancelClicked)
				]
			]
		]
	];

	// Load cloud settings and query namespaces after UI is constructed
	if (!LoadCloudSettings())
	{
		// Error already logged and status text already set by LoadCloudSettings()
		// LoadCloudSettings() validates zen.exe and OidcToken.exe paths
		// StatusTextBlock already contains specific error (e.g., "zen.exe not found")
		// Abort initialization to prevent cascade failures
		return;
	}

	// Query namespaces (only if settings loaded successfully)
	QueryNamespaces();
}

bool SCloudDownloadDialog::LoadCloudSettings()
{
	// Find zen.exe and OidcToken.exe using unified config helpers
	IoStoreConfig::FindZenExePath(ZenExePath);
	IoStoreConfig::FindOidcTokenExePath(OidcExePath, ZenExePath);

	// Validate paths
	if (!FPaths::FileExists(ZenExePath))
	{
		UE_LOGF(LogTemp, Error, "zen.exe not found at: %ls", *ZenExePath);
		UE_LOGF(LogTemp, Error, "Searched locations:");
		UE_LOGF(LogTemp, Error, "  1. Config: [IoStoreDependencyViewer] ZenExePath");
		UE_LOGF(LogTemp, Error, "  2. Executable directory: %ls", *FPaths::Combine(FPlatformProcess::BaseDir(), TEXT("zen.exe")));
		FString ProjectDir;
		if (FParse::Value(FCommandLine::Get(), TEXT("projectdir="), ProjectDir))
		{
			UE_LOGF(LogTemp, Error, "  3. Project directory: %ls", *FPaths::Combine(ProjectDir, TEXT("Engine\\Binaries\\Win64\\zen.exe")));
		}

		if (StatusTextBlock.IsValid())
		{
			StatusTextBlock->SetText(FText::FromString(TEXT("Error: zen.exe not found. Check logs for search locations.")));
			StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
		}
		return false;
	}

	if (!FPaths::FileExists(OidcExePath))
	{
		UE_LOGF(LogTemp, Error, "OidcToken.exe not found at: %ls", *OidcExePath);
		UE_LOGF(LogTemp, Error, "Note: OidcToken.exe is required for authentication. Set OidcTokenExePath in Engine.ini if using a custom location.");

		if (StatusTextBlock.IsValid())
		{
			StatusTextBlock->SetText(FText::FromString(TEXT("Error: OidcToken.exe not found. Check logs for details.")));
			StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
		}
		return false;
	}

	// Use default host for cloud builds
	// TODO: Read from Engine.ini if there's a specific config location for this
	ProxyUrl = TEXT("https://jupiter.devtools.epicgames.com");

	UE_LOGF(LogTemp, Log, "Cloud settings loaded: ZenExe=%ls, OidcExe=%ls, Host=%ls", *ZenExePath, *OidcExePath, *ProxyUrl);

	return true;
}

bool SCloudDownloadDialog::QueryNamespaces()
{
	if (StatusTextBlock.IsValid())
	{
		StatusTextBlock->SetText(FText::FromString(TEXT("Querying namespaces...")));
		StatusTextBlock->SetColorAndOpacity(FLinearColor::Yellow);
	}

	// Ensure intermediate directory exists before creating temp file path
	FString IntermediateDir = FPaths::ProjectIntermediateDir();
	if (!IFileManager::Get().DirectoryExists(*IntermediateDir))
	{
		IFileManager::Get().MakeDirectory(*IntermediateDir, true);
	}

	FString TempResultPath = FPaths::CreateTempFilename(*IntermediateDir, TEXT("namespaces_"), TEXT(".json"));
	FString Command = FString::Printf(
		TEXT("builds list-namespaces --recursive --host \"%s\" --oidctoken-exe-path \"%s\" --result-path \"%s\""),
		*IoStoreConfig::EscapeCommandLineArgument(ProxyUrl),
		*IoStoreConfig::EscapeCommandLineArgument(OidcExePath),
		*IoStoreConfig::EscapeCommandLineArgument(TempResultPath)
	);

	FString Result;
	int32 ExitCode = RunZenCommand(Command, Result);

	if (ExitCode != 0 || !FPaths::FileExists(TempResultPath))
	{
		UE_LOGF(LogTemp, Error, "Failed to query namespaces. Exit code: %d", ExitCode);
		if (StatusTextBlock.IsValid())
		{
			StatusTextBlock->SetText(FText::FromString(TEXT("Error: Failed to query namespaces")));
			StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
		}
		IFileManager::Get().Delete(*TempResultPath);
		return false;
	}

	// Parse JSON result
	FString JsonString;
	if (!FFileHelper::LoadFileToString(JsonString, *TempResultPath))
	{
		UE_LOGF(LogTemp, Error, "Failed to read namespace result file");
		if (StatusTextBlock.IsValid())
		{
			StatusTextBlock->SetText(FText::FromString(TEXT("Error: Failed to read namespace result")));
			StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
		}
		IFileManager::Get().Delete(*TempResultPath);
		return false;
	}

	TSharedPtr<FJsonObject> JsonObject;
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);

	// Clear bucket map
	NamespaceBuckets.Empty();

	if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
	{
		NamespaceList.Empty();

		const TArray<TSharedPtr<FJsonValue>>* Results;
		if (JsonObject->TryGetArrayField(TEXT("results"), Results))
		{
			for (const TSharedPtr<FJsonValue>& ResultValue : *Results)
			{
				const TSharedPtr<FJsonObject>* ResultObj;
				if (ResultValue->TryGetObject(ResultObj))
				{
					FString NamespaceName;
					if ((*ResultObj)->TryGetStringField(TEXT("name"), NamespaceName))
					{
						NamespaceList.Add(MakeShared<FString>(NamespaceName));

						// Parse buckets (items array)
						TArray<FString> Buckets;
						const TArray<TSharedPtr<FJsonValue>>* Items;
						if ((*ResultObj)->TryGetArrayField(TEXT("items"), Items))
						{
							for (const TSharedPtr<FJsonValue>& ItemValue : *Items)
							{
								// Safely extract string value - TryGetString handles type validation
								FString BucketName;
								if (ItemValue.IsValid() && ItemValue->TryGetString(BucketName) && !BucketName.IsEmpty())
								{
									Buckets.Add(BucketName);
								}
							}
						}

						NamespaceBuckets.Add(NamespaceName, Buckets);
						UE_LOGF(LogTemp, Log, "Namespace %ls has %d buckets", *NamespaceName, Buckets.Num());
					}
				}
			}
		}

		if (NamespaceList.Num() > 0)
		{
			SelectedNamespace = NamespaceList[0];
			if (NamespaceComboBox.IsValid())
			{
				NamespaceComboBox->RefreshOptions();
			}
			if (StatusTextBlock.IsValid())
			{
				StatusTextBlock->SetText(FText::FromString(FString::Printf(TEXT("Found %d namespaces"), NamespaceList.Num())));
				StatusTextBlock->SetColorAndOpacity(FLinearColor::Green);
			}

			// Cleanup and return success
			IFileManager::Get().Delete(*TempResultPath);
			return true;
		}
		else
		{
			// JSON parsed successfully but no namespaces found
			UE_LOGF(LogTemp, Error, "JSON parsed successfully but no namespaces found in response");
			if (StatusTextBlock.IsValid())
			{
				StatusTextBlock->SetText(FText::FromString(TEXT("Error: No namespaces found in response")));
				StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
			}
			IFileManager::Get().Delete(*TempResultPath);
			return false;
		}
	}
	else
	{
		// JSON deserialization failed
		UE_LOGF(LogTemp, Error, "Failed to deserialize namespace JSON response");
		if (StatusTextBlock.IsValid())
		{
			StatusTextBlock->SetText(FText::FromString(TEXT("Error: Failed to parse namespace response")));
			StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
		}
		IFileManager::Get().Delete(*TempResultPath);
		return false;
	}
}

bool SCloudDownloadDialog::QueryBuilds(const FString& Namespace, const FString& BuildType)
{
	if (Namespace.IsEmpty())
	{
		if (StatusTextBlock.IsValid())
		{
			StatusTextBlock->SetText(FText::FromString(TEXT("Error: No namespace selected")));
			StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
		}
		return false;
	}

	if (StatusTextBlock.IsValid())
	{
		StatusTextBlock->SetText(FText::FromString(TEXT("Querying builds...")));
		StatusTextBlock->SetColorAndOpacity(FLinearColor::Yellow);
	}

	// Get matching buckets based on selected filters
	TArray<FString> MatchingBuckets = GetMatchingBuckets();
	if (MatchingBuckets.Num() == 0)
	{
		UE_LOGF(LogTemp, Error, "No matching buckets found for selected criteria");
		if (StatusTextBlock.IsValid())
		{
			StatusTextBlock->SetText(FText::FromString(TEXT("Error: No matching buckets found")));
			StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
		}
		return false;
	}

	UE_LOGF(LogTemp, Log, "Querying %d matching buckets in namespace: %ls", MatchingBuckets.Num(), *Namespace);

	BuildList.Empty();
	// Use composite key (BuildGroup + BuildId + Bucket) to avoid merging distinct builds
	TMap<FString, TSharedPtr<FCloudBuild>> BuildGroups;

	// Ensure intermediate directory exists before creating temp file paths
	FString IntermediateDir = FPaths::ProjectIntermediateDir();
	if (!IFileManager::Get().DirectoryExists(*IntermediateDir))
	{
		IFileManager::Get().MakeDirectory(*IntermediateDir, true);
	}

	// Query each matching bucket
	for (const FString& Bucket : MatchingBuckets)
	{

		UE_LOGF(LogTemp, Log, "Querying bucket: %ls", *Bucket);

		// Run zen.exe builds list --bucket <bucket>
		FString TempResultPath = FPaths::CreateTempFilename(*IntermediateDir, TEXT("builds_"), TEXT(".json"));
		FString Command = FString::Printf(
			TEXT("builds list --namespace=\"%s\" --bucket \"%s\" --host \"%s\" --oidctoken-exe-path \"%s\" --result-path \"%s\""),
			*IoStoreConfig::EscapeCommandLineArgument(Namespace),
			*IoStoreConfig::EscapeCommandLineArgument(Bucket),
			*IoStoreConfig::EscapeCommandLineArgument(ProxyUrl),
			*IoStoreConfig::EscapeCommandLineArgument(OidcExePath),
			*IoStoreConfig::EscapeCommandLineArgument(TempResultPath)
		);

		FString Result;
		int32 ExitCode = RunZenCommand(Command, Result);

		if (ExitCode != 0 || !FPaths::FileExists(TempResultPath))
		{
			UE_LOGF(LogTemp, Warning, "Failed to query bucket %ls. Exit code: %d", *Bucket, ExitCode);
			IFileManager::Get().Delete(*TempResultPath);
			continue; // Try next bucket
		}

		// Parse JSON result for this bucket
		FString JsonString;
		if (!FFileHelper::LoadFileToString(JsonString, *TempResultPath))
		{
			UE_LOGF(LogTemp, Warning, "Failed to read result file for bucket %ls", *Bucket);
			IFileManager::Get().Delete(*TempResultPath);
			continue;
		}

		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);

		if (FJsonSerializer::Deserialize(Reader, JsonObject) && JsonObject.IsValid())
		{
			const TArray<TSharedPtr<FJsonValue>>* Results;
			if (JsonObject->TryGetArrayField(TEXT("results"), Results))
			{
				UE_LOGF(LogTemp, Log, "  Bucket %ls returned %d builds", *Bucket, Results->Num());

				for (const TSharedPtr<FJsonValue>& ResultValue : *Results)
			{
					const TSharedPtr<FJsonObject>* ResultObj;
					if (ResultValue->TryGetObject(ResultObj))
					{
						FString BuildId;
						(*ResultObj)->TryGetStringField(TEXT("buildId"), BuildId);

						const TSharedPtr<FJsonObject>* MetadataObj;
						if ((*ResultObj)->TryGetObjectField(TEXT("metadata"), MetadataObj))
						{
							FString BuildGroup, PlatformName;
							(*MetadataObj)->TryGetStringField(TEXT("buildgroup"), BuildGroup);
							(*MetadataObj)->TryGetStringField(TEXT("name"), PlatformName);

							if (!BuildGroup.IsEmpty())
							{
								// Use composite key to distinguish builds with same BuildGroup but different BuildId/Bucket
								FString CompositeKey = FString::Printf(TEXT("%s|%s|%s"), *BuildGroup, *BuildId, *Bucket);
								TSharedPtr<FCloudBuild>* ExistingBuild = BuildGroups.Find(CompositeKey);
								if (ExistingBuild)
								{
									// Add platform to existing build (same BuildGroup, BuildId, and Bucket)
									if (!PlatformName.IsEmpty() && !(*ExistingBuild)->Platforms.Contains(PlatformName))
									{
										(*ExistingBuild)->Platforms.Add(PlatformName);
									}
								}
								else
								{
									// Create new build entry
									FString CreatedAtStr;
									(*MetadataObj)->TryGetStringField(TEXT("createdAt"), CreatedAtStr);
									FDateTime CreatedAt;
									FDateTime::ParseIso8601(*CreatedAtStr, CreatedAt);

									FString Commit;
									(*MetadataObj)->TryGetStringField(TEXT("commit"), Commit);

									TSharedPtr<FCloudBuild> NewBuild = MakeShared<FCloudBuild>(
										BuildGroup, BuildId, Commit, Bucket, Namespace, CreatedAt
									);

									if (!PlatformName.IsEmpty())
									{
										NewBuild->Platforms.Add(PlatformName);
									}

									BuildGroups.Add(CompositeKey, NewBuild);
								}
							}
						}
					}
				}
			}
		}

		// Cleanup this bucket's result file
		IFileManager::Get().Delete(*TempResultPath);
	}

	// Convert map to array and sort by CL number (descending - newest first)
	for (auto& Pair : BuildGroups)
	{
		BuildList.Add(Pair.Value);
		UE_LOGF(LogTemp, Log, "  Build: %ls, Platforms: %d", *Pair.Value->BuildName, Pair.Value->Platforms.Num());
	}

	BuildList.Sort([](const TSharedPtr<FCloudBuild>& A, const TSharedPtr<FCloudBuild>& B)
	{
		// Sort by CL number descending (newest first)
		int32 CL_A = FCString::Atoi(*A->Changelist);
		int32 CL_B = FCString::Atoi(*B->Changelist);
		return CL_A > CL_B;
	});

	// Apply filter to populate FilteredBuildList (also handles SelectedBuild intelligently)
	ApplyBuildFilter();

	if (BuildList.Num() > 0)
	{
		// ApplyBuildFilter() already set SelectedBuild to the first filtered build (or preserved existing selection)
		// Don't overwrite it with BuildList[0] - that would break filtering
		if (BuildComboBox.IsValid())
		{
			BuildComboBox->RefreshOptions();
		}
		if (StatusTextBlock.IsValid())
		{
			StatusTextBlock->SetText(FText::FromString(FString::Printf(TEXT("Found %d builds"), BuildList.Num())));
			StatusTextBlock->SetColorAndOpacity(FLinearColor::Green);
		}
		UE_LOGF(LogTemp, Log, "Successfully found %d build groups", BuildList.Num());
	}
	else
	{
		if (StatusTextBlock.IsValid())
		{
			StatusTextBlock->SetText(FText::FromString(TEXT("No builds found - check logs")));
			StatusTextBlock->SetColorAndOpacity(FLinearColor::Yellow);
		}
		UE_LOGF(LogTemp, Warning, "No build groups created from results");
	}

	return true;
}

bool SCloudDownloadDialog::DownloadBuild(const FCloudBuild& Build)
{
	// Reset cancel flag at start of download
	bCancelRequested = false;

	if (StatusTextBlock.IsValid())
	{
		StatusTextBlock->SetText(FText::FromString(TEXT("Starting download...")));
		StatusTextBlock->SetColorAndOpacity(FLinearColor::Yellow);
	}

	// Create download directory if it doesn't exist
	if (!IFileManager::Get().DirectoryExists(*DownloadDirectory))
	{
		if (!IFileManager::Get().MakeDirectory(*DownloadDirectory, true))
		{
			FString ErrorMsg = FString::Printf(TEXT("Failed to create download directory: %s"), *DownloadDirectory);
			UE_LOGF(LogTemp, Error, "%ls", *ErrorMsg);
			if (StatusTextBlock.IsValid())
			{
				StatusTextBlock->SetText(FText::FromString(ErrorMsg));
				StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
			}
			return false;
		}
	}

	// Check if partial downloads are enabled
	bool bUsePartialDownloads = UsePartialDownloadsCheckBox.IsValid() && UsePartialDownloadsCheckBox->IsChecked();

	// Determine wildcard based on partial download setting
	FString Wildcard;
	FString ExcludeWildcard;
	if (bUsePartialDownloads)
	{
		// For partial downloads: download only metadata files (.utoc, .uondemandtoc, .umeta)
		// Scope wildcard to IoStore file types to prevent --clean from deleting non-IoStore files
		Wildcard = TEXT("*.utoc;*.uondemandtoc;*.umeta");
		ExcludeWildcard = TEXT("");
		UE_LOGF(LogTemp, Log, "Using partial downloads - downloading metadata only");
	}
	else
	{
		// For full downloads: download all IoStore files
		// Scope wildcard to IoStore file types to prevent --clean from deleting non-IoStore files
		Wildcard = TEXT("*.utoc;*.ucas;*.uondemandtoc;*.umeta");
		ExcludeWildcard = TEXT("");
	}

	// Build download command
	// --clean: Remove files not in the build (prevents stale files across builds)
	// --enable-scavenge: Resume interrupted downloads by reusing existing chunks
	// Note: File locking is prevented by calling ClearAllData() before downloads to close open file handles
	FString Command;
	if (!ExcludeWildcard.IsEmpty())
	{
		Command = FString::Printf(
			TEXT("builds download --namespace=\"%s\" --bucket=\"%s\" --build-id=\"%s\" --local-path=\"%s\" --host=\"%s\" --oidctoken-exe-path=\"%s\" --wildcard=\"%s\" --exclude-wildcard=\"%s\" --clean --enable-scavenge --verbose"),
			*IoStoreConfig::EscapeCommandLineArgument(Build.Namespace),
			*IoStoreConfig::EscapeCommandLineArgument(Build.BucketId),
			*IoStoreConfig::EscapeCommandLineArgument(Build.BuildId),
			*IoStoreConfig::EscapeCommandLineArgument(DownloadDirectory),
			*IoStoreConfig::EscapeCommandLineArgument(ProxyUrl),
			*IoStoreConfig::EscapeCommandLineArgument(OidcExePath),
			*Wildcard,
			*ExcludeWildcard
		);
	}
	else
	{
		Command = FString::Printf(
			TEXT("builds download --namespace=\"%s\" --bucket=\"%s\" --build-id=\"%s\" --local-path=\"%s\" --host=\"%s\" --oidctoken-exe-path=\"%s\" --wildcard=\"%s\" --clean --enable-scavenge --verbose"),
			*IoStoreConfig::EscapeCommandLineArgument(Build.Namespace),
			*IoStoreConfig::EscapeCommandLineArgument(Build.BucketId),
			*IoStoreConfig::EscapeCommandLineArgument(Build.BuildId),
			*IoStoreConfig::EscapeCommandLineArgument(DownloadDirectory),
			*IoStoreConfig::EscapeCommandLineArgument(ProxyUrl),
			*IoStoreConfig::EscapeCommandLineArgument(OidcExePath),
			*Wildcard
		);
	}

	UE_LOGF(LogTemp, Display, "Executing zen.exe download:");
	UE_LOGF(LogTemp, Display, "  Wildcard: %ls", *Wildcard);
	UE_LOGF(LogTemp, Display, "  Exclude wildcard: %ls", ExcludeWildcard.IsEmpty() ? TEXT("<none>") : *ExcludeWildcard);
	UE_LOGF(LogTemp, Display, "  Partial downloads enabled: %ls", bUsePartialDownloads ? TEXT("YES") : TEXT("NO"));
	UE_LOGF(LogTemp, Display, "  Download directory: %ls", *DownloadDirectory);
	UE_LOGF(LogTemp, Display, "  Full command: %ls %ls", *ZenExePath, *Command);

	FString Result;
	int32 ExitCode = RunZenCommand(Command, Result);

	UE_LOGF(LogTemp, Display, "zen.exe download completed with exit code: %d", ExitCode);

	if (ExitCode != 0)
	{
		// Don't log error or overwrite status if this was a user cancellation
		if (!bCancelRequested)
		{
			UE_LOGF(LogTemp, Error, "Download failed. Exit code: %d", ExitCode);
			UE_LOGF(LogTemp, Error, "zen.exe output (last 1000 chars): %ls", *Result.Right(1000));
			if (StatusTextBlock.IsValid())
			{
				StatusTextBlock->SetText(FText::FromString(TEXT("Error: Download failed")));
				StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
			}
		}
		return false;
	}

	// Verify files were actually downloaded
	TArray<FString> DownloadedUtocFiles;
	IFileManager::Get().FindFilesRecursive(DownloadedUtocFiles, *DownloadDirectory, TEXT("*.utoc"), true, false);
	UE_LOGF(LogTemp, Display, "Found %d .utoc files after download", DownloadedUtocFiles.Num());
	if (DownloadedUtocFiles.Num() == 0)
	{
		UE_LOGF(LogTemp, Error, "No .utoc files found after download - command may have failed silently");
		UE_LOGF(LogTemp, Error, "Download directory: %ls", *DownloadDirectory);
		UE_LOGF(LogTemp, Error, "Wildcard used: %ls", *Wildcard);
		UE_LOGF(LogTemp, Error, "Exclude wildcard: %ls", ExcludeWildcard.IsEmpty() ? TEXT("<none>") : *ExcludeWildcard);

		if (StatusTextBlock.IsValid())
		{
			StatusTextBlock->SetText(FText::FromString(TEXT("Download failed: No .utoc files found. Check logs for details.")));
			StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
		}

		return false;
	}

	// Partial downloads are handled by FIoStoreReaderZenBuild in LoadAllContainersWithProgress
	// No need for platform file wrapper - zen reader handles on-demand fetching directly
	if (StatusTextBlock.IsValid())
	{
		if (bUsePartialDownloads)
		{
			StatusTextBlock->SetText(FText::FromString(TEXT("Metadata download complete! On-demand fetching will be used.")));
		}
		else
		{
			StatusTextBlock->SetText(FText::FromString(TEXT("Download completed!")));
		}
		StatusTextBlock->SetColorAndOpacity(FLinearColor::Green);
	}

	bDownloadInitiated = true;
	bPartialDownloadUsed = bUsePartialDownloads;

	// Save download.ini file with build information for future reference
	FString DownloadIniPath = FPaths::Combine(DownloadDirectory, TEXT("download.ini"));
	FConfigFile DownloadConfig;

	DownloadConfig.SetString(TEXT("CloudBuild"), TEXT("BuildName"), *Build.BuildName);
	DownloadConfig.SetString(TEXT("CloudBuild"), TEXT("BuildId"), *Build.BuildId);
	DownloadConfig.SetString(TEXT("CloudBuild"), TEXT("Changelist"), *Build.Changelist);
	DownloadConfig.SetString(TEXT("CloudBuild"), TEXT("Namespace"), *Build.Namespace);
	DownloadConfig.SetString(TEXT("CloudBuild"), TEXT("BucketId"), *Build.BucketId);
	DownloadConfig.SetString(TEXT("CloudBuild"), TEXT("Host"), *ProxyUrl);
	DownloadConfig.SetString(TEXT("CloudBuild"), TEXT("CreatedAt"), *Build.CreatedAt.ToString());
	DownloadConfig.SetString(TEXT("CloudBuild"), TEXT("PartialDownloadsEnabled"), bUsePartialDownloads ? TEXT("true") : TEXT("false"));

	if (Build.Platforms.Num() > 0)
	{
		DownloadConfig.SetString(TEXT("CloudBuild"), TEXT("Platforms"), *FString::Join(Build.Platforms, TEXT(", ")));
	}

	if (!DownloadConfig.Write(DownloadIniPath))
	{
		UE_LOGF(LogTemp, Warning, "Failed to write download.ini to: %ls", *DownloadIniPath);
	}
	else
	{
		UE_LOGF(LogTemp, Display, "Saved download information to: %ls", *DownloadIniPath);
	}

	// Try to load crypto.json from configured search paths
	// Configured in Engine.ini [IoStoreDependencyViewer] +CryptoJsonSearchPath with template variables
	FString ChangelistNumber = IoStoreConfig::ParseChangelistFromBuildName(Build.BuildName);
	for (const FString& Platform : Build.Platforms)
	{
		TMap<FString, FString> TemplateVars;
		TemplateVars.Add(TEXT("buildname"), Build.BuildName);
		TemplateVars.Add(TEXT("branchname"), IoStoreConfig::ParseBranchFromBuildName(Build.BuildName));
		TemplateVars.Add(TEXT("cl"), ChangelistNumber.IsEmpty() ? Build.Changelist : ChangelistNumber);
		TemplateVars.Add(TEXT("platform"), Platform);
		IoStoreConfig::TryLoadCryptoJsonFromSearchPaths(TemplateVars, DownloadDirectory);
	}

	return true;
}

int32 SCloudDownloadDialog::RunZenCommand(const FString& Command, FString& OutResult)
{
	FString FullCommand = FString::Printf(TEXT("\"%s\" %s"), *ZenExePath, *Command);

	UE_LOGF(LogTemp, Display, "Running: %ls", *FullCommand);

	void* ReadPipe = nullptr;
	void* WritePipe = nullptr;
	if (!FPlatformProcess::CreatePipe(ReadPipe, WritePipe))
	{
		UE_LOGF(LogTemp, Error, "Failed to create pipe for zen.exe process (platform limitation or resource exhaustion)");
		return -1;
	}

	FProcHandle ProcHandle = FPlatformProcess::CreateProc(
		*ZenExePath,
		*Command,
		false,
		true,
		true,
		nullptr,
		0,
		nullptr,
		WritePipe,
		ReadPipe
	);

	if (!ProcHandle.IsValid())
	{
		UE_LOGF(LogTemp, Error, "Failed to launch zen.exe");
		FPlatformProcess::ClosePipe(ReadPipe, WritePipe);
		return -1;
	}

	// Close parent's copy of write pipe - child has its own copy
	// This ensures proper EOF detection when child closes its write handle
	FPlatformProcess::ClosePipe(0, WritePipe);

	// Create weak pointer to detect if this dialog is destroyed during Slate pumping
	TWeakPtr<SCloudDownloadDialog> WeakThis = SharedThis(this);

	// Track progress
	double StartTime = FPlatformTime::Seconds();
	double LastProgressTime = StartTime;
	double LastStatusUpdateTime = StartTime;
	int32 ProgressCounter = 0;
	int32 LastKnownPercentage = 0;

	// Wait for process to complete with 2-hour timeout (full downloads of large builds can take hours)
	const double TimeoutSeconds = 2.0 * 60.0 * 60.0;
	while (FPlatformProcess::IsProcRunning(ProcHandle))
	{
		// Check for timeout
		double CurrentTime = FPlatformTime::Seconds();
		double ElapsedTime = CurrentTime - StartTime;
		if (ElapsedTime > TimeoutSeconds)
		{
			UE_LOGF(LogTemp, Error, "zen.exe operation timed out after %.1f hours", ElapsedTime / 3600.0);
			if (StatusTextBlock.IsValid())
			{
				StatusTextBlock->SetText(FText::FromString(TEXT("Error: Operation timed out")));
				StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
			}
			FPlatformProcess::TerminateProc(ProcHandle, true);
			FPlatformProcess::ClosePipe(ReadPipe, 0);  // WritePipe already closed
			FPlatformProcess::CloseProc(ProcHandle);  // Close process handle to avoid leak
			return -1;
		}

		FPlatformProcess::Sleep(0.05f);  // Reduced sleep for more responsive UI

		// Read output
		FString NewOutput = FPlatformProcess::ReadPipe(ReadPipe);
		if (!NewOutput.IsEmpty())
		{
			OutResult += NewOutput;
			// Limit OutResult size to prevent OOM on very verbose/long-running commands
			// Keep only the last 100KB of output (approximately 50,000 characters)
			const int32 MaxOutputSize = 50000;
			if (OutResult.Len() > MaxOutputSize)
			{
				OutResult = OutResult.Right(MaxOutputSize);
			}
			// Use Display level so output is visible in logs
			UE_LOGF(LogTemp, Display, "%ls", *NewOutput.TrimStartAndEnd());
			LastProgressTime = FPlatformTime::Seconds();
			ProgressCounter = 0;

			// Try to parse percentage from zen.exe output
			// Format: "Writing chunks    27% (40.3s): ..."
			int32 PercentIndex = NewOutput.Find(TEXT("%"));
			if (PercentIndex != INDEX_NONE && PercentIndex > 0)
			{
				// Search backwards for the percentage number
				int32 StartIndex = PercentIndex - 1;
				while (StartIndex > 0 && (FChar::IsDigit(NewOutput[StartIndex]) || FChar::IsWhitespace(NewOutput[StartIndex])))
				{
					StartIndex--;
				}
				StartIndex++; // Move back to first digit

				FString PercentStr = NewOutput.Mid(StartIndex, PercentIndex - StartIndex).TrimStartAndEnd();
				if (!PercentStr.IsEmpty() && PercentStr.IsNumeric())
				{
					LastKnownPercentage = FCString::Atoi(*PercentStr);
				}
			}

			// Update status text with progress
			FString StatusMessage;
			if (LastKnownPercentage > 0)
			{
				StatusMessage = FString::Printf(TEXT("Downloading... %d%%"), LastKnownPercentage);
			}
			else
			{
				StatusMessage = TEXT("Downloading...");
			}

			if (StatusTextBlock.IsValid())
			{
				StatusTextBlock->SetText(FText::FromString(StatusMessage));
				StatusTextBlock->SetColorAndOpacity(FLinearColor::Yellow);
			}

			LastStatusUpdateTime = FPlatformTime::Seconds();
		}
		else
		{
			// Show progress indicator even with no new output
			if (CurrentTime - LastStatusUpdateTime > 1.0)
			{
				int32 ElapsedSeconds = (int32)(CurrentTime - StartTime);
				int32 Minutes = ElapsedSeconds / 60;
				int32 Seconds = ElapsedSeconds % 60;

				const TCHAR* AnimChars[] = { TEXT("|"), TEXT("/"), TEXT("-"), TEXT("\\") };
				FString ProgressText;
				if (LastKnownPercentage > 0)
				{
					ProgressText = FString::Printf(TEXT("Downloading... %d%% %s %02d:%02d"),
						LastKnownPercentage, AnimChars[ProgressCounter % 4], Minutes, Seconds);
				}
				else
				{
					ProgressText = FString::Printf(TEXT("Downloading... %s %02d:%02d"),
						AnimChars[ProgressCounter % 4], Minutes, Seconds);
				}

				if (StatusTextBlock.IsValid())
				{
					StatusTextBlock->SetText(FText::FromString(ProgressText));
				}

				LastStatusUpdateTime = CurrentTime;
				ProgressCounter++;
			}
		}

		// Check if user requested cancellation
		if (bCancelRequested)
		{
			UE_LOGF(LogTemp, Warning, "User cancelled download - terminating zen.exe");
			FPlatformProcess::TerminateProc(ProcHandle, true);
			FPlatformProcess::ClosePipe(ReadPipe, 0);  // WritePipe already closed
			FPlatformProcess::CloseProc(ProcHandle);

			if (StatusTextBlock.IsValid())
			{
				StatusTextBlock->SetText(FText::FromString(TEXT("Download cancelled by user")));
				StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
			}

			return -1;
		}

		// Keep UI responsive
		// NOTE: Pumping Slate messages can trigger callbacks that might destroy this dialog
		FSlateApplication::Get().PumpMessages();
		FSlateApplication::Get().Tick();

		// Check if dialog was destroyed during Slate pumping
		if (!WeakThis.IsValid())
		{
			// Dialog was destroyed - clean up process and exit
			UE_LOGF(LogTemp, Warning, "Cloud download dialog was closed during zen.exe operation - terminating process");
			FPlatformProcess::TerminateProc(ProcHandle, true);
			FPlatformProcess::ClosePipe(ReadPipe, 0);  // WritePipe already closed
			FPlatformProcess::CloseProc(ProcHandle);
			return -1;
		}
	}

	// Read any remaining output
	FString FinalOutput = FPlatformProcess::ReadPipe(ReadPipe);
	if (!FinalOutput.IsEmpty())
	{
		OutResult += FinalOutput;
		// Apply same size limit to final output
		const int32 MaxOutputSize = 50000;
		if (OutResult.Len() > MaxOutputSize)
		{
			OutResult = OutResult.Right(MaxOutputSize);
		}
		UE_LOGF(LogTemp, Display, "%ls", *FinalOutput.TrimStartAndEnd());
	}

	int32 ReturnCode = 0;
	FPlatformProcess::GetProcReturnCode(ProcHandle, &ReturnCode);

	double TotalTime = FPlatformTime::Seconds() - StartTime;
	int32 TotalMinutes = (int32)TotalTime / 60;
	int32 TotalSeconds = (int32)TotalTime % 60;
	UE_LOGF(LogTemp, Display, "zen.exe completed in %02d:%02d (exit code: %d)",
		TotalMinutes, TotalSeconds, ReturnCode);

	FPlatformProcess::CloseProc(ProcHandle);
	FPlatformProcess::ClosePipe(ReadPipe, 0);  // WritePipe already closed

	// Give zen.exe background threads time to finish file operations before returning
	// This prevents file sharing violations when scanning for .utoc files immediately after download
	if (ReturnCode == 0)
	{
		UE_LOGF(LogTemp, Display, "Waiting for zen.exe background operations to complete...");
		FPlatformProcess::Sleep(2.0f);
	}

	return ReturnCode;
}

// UI Callbacks

FReply SCloudDownloadDialog::OnRefreshNamespacesClicked()
{
	QueryNamespaces();
	return FReply::Handled();
}

FReply SCloudDownloadDialog::OnQueryBuildsClicked()
{
	if (SelectedNamespace.IsValid() && SelectedBuildType.IsValid())
	{
		QueryBuilds(*SelectedNamespace, *SelectedBuildType);
	}
	return FReply::Handled();
}

FReply SCloudDownloadDialog::OnDownloadClicked()
{
	if (SelectedBuild.IsValid())
	{
		bool bDownloadSucceeded = DownloadBuild(*SelectedBuild);

		// Close dialog after successful download OR after user cancellation
		if (bDownloadSucceeded || bCancelRequested)
		{
			if (TSharedPtr<SWindow> ParentWindowPtr = ParentWindow.Pin())
			{
				FSlateApplication::Get().RequestDestroyWindow(ParentWindowPtr.ToSharedRef());
			}
		}
	}
	else
	{
		if (StatusTextBlock.IsValid())
		{
			StatusTextBlock->SetText(FText::FromString(TEXT("Error: No build selected")));
			StatusTextBlock->SetColorAndOpacity(FLinearColor::Red);
		}
	}
	return FReply::Handled();
}

FReply SCloudDownloadDialog::OnCancelClicked()
{
	// Set cancel flag - RunZenCommand will detect this and terminate the process if a download is active
	bCancelRequested = true;

	// Close the dialog window
	if (TSharedPtr<SWindow> ParentWindowPtr = ParentWindow.Pin())
	{
		FSlateApplication::Get().RequestDestroyWindow(ParentWindowPtr.ToSharedRef());
	}

	return FReply::Handled();
}

FReply SCloudDownloadDialog::OnBrowseDirectoryClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		FString SelectedDirectory;
		if (DesktopPlatform->OpenDirectoryDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			TEXT("Select Download Directory"),
			DownloadDirectory,
			SelectedDirectory))
		{
			DownloadDirectory = SelectedDirectory;
			DownloadDirectoryTextBox->SetText(FText::FromString(DownloadDirectory));
		}
	}
	return FReply::Handled();
}

// Dropdown callbacks

TSharedRef<SWidget> SCloudDownloadDialog::OnGenerateNamespaceWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock).Text(FText::FromString(*InItem));
}

void SCloudDownloadDialog::OnNamespaceSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	SelectedNamespace = NewSelection;
	UpdateAvailableStreams();
}

FText SCloudDownloadDialog::GetCurrentNamespaceText() const
{
	return SelectedNamespace.IsValid() ? FText::FromString(*SelectedNamespace) : FText::FromString(TEXT("Select Namespace"));
}

TSharedRef<SWidget> SCloudDownloadDialog::OnGenerateBuildTypeWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock).Text(FText::FromString(*InItem));
}

void SCloudDownloadDialog::OnBuildTypeSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	SelectedBuildType = NewSelection;
	UpdateAvailableStreams();
}

FText SCloudDownloadDialog::GetCurrentBuildTypeText() const
{
	return SelectedBuildType.IsValid() ? FText::FromString(*SelectedBuildType) : FText::FromString(TEXT("Select Build Type"));
}

TSharedRef<SWidget> SCloudDownloadDialog::OnGenerateBuildWidget(TSharedPtr<FCloudBuild> InItem)
{
	FString DisplayText = FString::Printf(
		TEXT("%s (CL %s) - %d platforms - %s"),
		*InItem->BuildName,
		*InItem->Changelist,
		InItem->Platforms.Num(),
		*InItem->CreatedAt.ToString()
	);
	return SNew(STextBlock).Text(FText::FromString(DisplayText));
}

void SCloudDownloadDialog::OnBuildSelectionChanged(TSharedPtr<FCloudBuild> NewSelection, ESelectInfo::Type SelectInfo)
{
	SelectedBuild = NewSelection;
}

FText SCloudDownloadDialog::GetCurrentBuildText() const
{
	if (SelectedBuild.IsValid())
	{
		return FText::FromString(FString::Printf(TEXT("%s (CL %s)"), *SelectedBuild->BuildName, *SelectedBuild->Changelist));
	}
	return FText::FromString(TEXT("Select Build"));
}

void SCloudDownloadDialog::OnDownloadDirectoryChanged(const FText& NewText)
{
	DownloadDirectory = NewText.ToString();
}

void SCloudDownloadDialog::OnBuildFilterTextChanged(const FText& NewText)
{
	BuildFilterText = NewText.ToString();
	ApplyBuildFilter();
}

TSharedRef<SWidget> SCloudDownloadDialog::OnGenerateStreamWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock).Text(FText::FromString(*InItem));
}

void SCloudDownloadDialog::OnStreamSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	// Save current platform selection to preserve it if still valid
	FString PreviousPlatform = SelectedPlatform.IsValid() ? *SelectedPlatform : FString();

	SelectedStream = NewSelection;
	UpdateAvailablePlatforms(PreviousPlatform);
}

FText SCloudDownloadDialog::GetCurrentStreamText() const
{
	return SelectedStream.IsValid() ? FText::FromString(*SelectedStream) : FText::FromString(TEXT("Select Stream"));
}

TSharedRef<SWidget> SCloudDownloadDialog::OnGeneratePlatformWidget(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock).Text(FText::FromString(*InItem));
}

void SCloudDownloadDialog::OnPlatformSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo)
{
	SelectedPlatform = NewSelection;
}

FText SCloudDownloadDialog::GetCurrentPlatformText() const
{
	return SelectedPlatform.IsValid() ? FText::FromString(*SelectedPlatform) : FText::FromString(TEXT("Select Platform"));
}

void SCloudDownloadDialog::UpdateAvailableStreams()
{
	// Save current selections to preserve them if still valid
	FString PreviousStream = SelectedStream.IsValid() ? *SelectedStream : FString();
	FString PreviousPlatform = SelectedPlatform.IsValid() ? *SelectedPlatform : FString();

	StreamList.Empty();
	PlatformList.Empty();
	SelectedStream.Reset();
	SelectedPlatform.Reset();

	if (!SelectedNamespace.IsValid() || !SelectedBuildType.IsValid())
	{
		if (StreamComboBox.IsValid())
		{
			StreamComboBox->RefreshOptions();
		}
		if (PlatformComboBox.IsValid())
		{
			PlatformComboBox->RefreshOptions();
		}
		return;
	}

	TArray<FString>* Buckets = NamespaceBuckets.Find(*SelectedNamespace);
	if (!Buckets)
	{
		if (StreamComboBox.IsValid())
		{
			StreamComboBox->RefreshOptions();
		}
		if (PlatformComboBox.IsValid())
		{
			PlatformComboBox->RefreshOptions();
		}
		return;
	}

	// Parse bucket format: {project}.{buildtype}.{stream}.{platform}
	TSet<FString> UniqueStreams;
	for (const FString& Bucket : *Buckets)
	{
		TArray<FString> Parts;
		Bucket.ParseIntoArray(Parts, TEXT("."));
		if (Parts.Num() >= 3 && Parts[1].Equals(*SelectedBuildType, ESearchCase::IgnoreCase))
		{
			UniqueStreams.Add(Parts[2]);
		}
	}

	// Sort and add to list
	TArray<FString> SortedStreams = UniqueStreams.Array();
	SortedStreams.Sort();
	for (const FString& Stream : SortedStreams)
	{
		StreamList.Add(MakeShared<FString>(Stream));
	}

	// Try to preserve previous selection if it's still valid
	if (!PreviousStream.IsEmpty())
	{
		for (const TSharedPtr<FString>& Stream : StreamList)
		{
			if (Stream.IsValid() && Stream->Equals(PreviousStream, ESearchCase::IgnoreCase))
			{
				SelectedStream = Stream;
				break;
			}
		}
	}

	// If previous selection wasn't found or was empty, pick the first one
	if (!SelectedStream.IsValid() && StreamList.Num() > 0)
	{
		SelectedStream = StreamList[0];
	}

	if (StreamComboBox.IsValid())
	{
		StreamComboBox->RefreshOptions();
	}

	UE_LOGF(LogTemp, Log, "Found %d streams for buildtype %ls", StreamList.Num(), **SelectedBuildType);
	UpdateAvailablePlatforms(PreviousPlatform);
}

void SCloudDownloadDialog::UpdateAvailablePlatforms(const FString& PreviousPlatform)
{
	PlatformList.Empty();
	SelectedPlatform.Reset();

	if (!SelectedNamespace.IsValid() || !SelectedBuildType.IsValid() || !SelectedStream.IsValid())
	{
		if (PlatformComboBox.IsValid())
		{
			PlatformComboBox->RefreshOptions();
		}
		return;
	}

	TArray<FString>* Buckets = NamespaceBuckets.Find(*SelectedNamespace);
	if (!Buckets)
	{
		if (PlatformComboBox.IsValid())
		{
			PlatformComboBox->RefreshOptions();
		}
		return;
	}

	// Parse bucket format: {project}.{buildtype}.{stream}.{platform}
	TSet<FString> UniquePlatforms;
	for (const FString& Bucket : *Buckets)
	{
		TArray<FString> Parts;
		Bucket.ParseIntoArray(Parts, TEXT("."));
		if (Parts.Num() >= 4 &&
			Parts[1].Equals(*SelectedBuildType, ESearchCase::IgnoreCase) &&
			Parts[2].Equals(*SelectedStream, ESearchCase::IgnoreCase))
		{
			UniquePlatforms.Add(Parts[3]);
		}
	}

	// Sort and add to list
	TArray<FString> SortedPlatforms = UniquePlatforms.Array();
	SortedPlatforms.Sort();
	for (const FString& Platform : SortedPlatforms)
	{
		PlatformList.Add(MakeShared<FString>(Platform));
	}

	// Try to preserve previous selection if it's still valid
	if (!PreviousPlatform.IsEmpty())
	{
		for (const TSharedPtr<FString>& Platform : PlatformList)
		{
			if (Platform.IsValid() && Platform->Equals(PreviousPlatform, ESearchCase::IgnoreCase))
			{
				SelectedPlatform = Platform;
				break;
			}
		}
	}

	// If previous selection wasn't found or was empty, pick the first one
	if (!SelectedPlatform.IsValid() && PlatformList.Num() > 0)
	{
		SelectedPlatform = PlatformList[0];
	}

	if (PlatformComboBox.IsValid())
	{
		PlatformComboBox->RefreshOptions();
	}

	UE_LOGF(LogTemp, Log, "Found %d platforms for buildtype %ls, stream %ls", PlatformList.Num(), **SelectedBuildType, **SelectedStream);
}

TArray<FString> SCloudDownloadDialog::GetMatchingBuckets() const
{
	TArray<FString> MatchingBuckets;

	if (!SelectedNamespace.IsValid() || !SelectedBuildType.IsValid())
	{
		return MatchingBuckets;
	}

	const TArray<FString>* Buckets = NamespaceBuckets.Find(*SelectedNamespace);
	if (!Buckets)
	{
		return MatchingBuckets;
	}

	// Filter buckets by selected criteria
	for (const FString& Bucket : *Buckets)
	{
		TArray<FString> Parts;
		Bucket.ParseIntoArray(Parts, TEXT("."));

		if (Parts.Num() < 3)
		{
			continue;
		}

		// Check buildtype
		if (!Parts[1].Equals(*SelectedBuildType, ESearchCase::IgnoreCase))
		{
			continue;
		}

		// Check stream if selected
		if (SelectedStream.IsValid() && !Parts[2].Equals(*SelectedStream, ESearchCase::IgnoreCase))
		{
			continue;
		}

		// Check platform if selected
		if (SelectedPlatform.IsValid())
		{
			// If platform filter is active, only include buckets with platform segment (Parts[3])
			if (Parts.Num() < 4)
			{
				// Bucket doesn't have platform segment - exclude when platform filter active
				continue;
			}

			if (!Parts[3].Equals(*SelectedPlatform, ESearchCase::IgnoreCase))
			{
				// Platform mismatch - exclude
				continue;
			}
		}

		MatchingBuckets.Add(Bucket);
	}

	return MatchingBuckets;
}

void SCloudDownloadDialog::ApplyBuildFilter()
{
	FilteredBuildList.Empty();

	if (BuildFilterText.IsEmpty())
	{
		// No filter - show all builds
		FilteredBuildList = BuildList;
	}
	else
	{
		// Apply regex filter
		FRegexPattern Pattern(BuildFilterText);

		for (const TSharedPtr<FCloudBuild>& Build : BuildList)
		{
			if (!Build.IsValid())
			{
				continue;
			}

			// Test against BuildName (which contains the CL number)
			FRegexMatcher Matcher(Pattern, Build->BuildName);
			if (Matcher.FindNext())
			{
				FilteredBuildList.Add(Build);
			}
		}
	}

	// Preserve selection if it's still in the filtered list
	bool bSelectionStillValid = false;
	if (SelectedBuild.IsValid())
	{
		for (const TSharedPtr<FCloudBuild>& Build : FilteredBuildList)
		{
			if (Build == SelectedBuild)
			{
				bSelectionStillValid = true;
				break;
			}
		}
	}

	// Select first build if current selection is not in filtered list
	if (!bSelectionStillValid && FilteredBuildList.Num() > 0)
	{
		SelectedBuild = FilteredBuildList[0];
	}
	else if (!bSelectionStillValid)
	{
		SelectedBuild.Reset();
	}

	// Refresh the combo box
	if (BuildComboBox.IsValid())
	{
		BuildComboBox->RefreshOptions();
	}
}
