// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include <atomic>

/** Represents a cloud build */
struct FCloudBuild
{
	FString BuildName;
	FString BuildId;
	FString Changelist;
	FString BucketId;
	FString Namespace;
	FDateTime CreatedAt;
	TArray<FString> Platforms;

	FCloudBuild() = default;
	FCloudBuild(const FString& InName, const FString& InBuildId, const FString& InCL, const FString& InBucket, const FString& InNamespace, const FDateTime& InDate)
		: BuildName(InName), BuildId(InBuildId), Changelist(InCL), BucketId(InBucket), Namespace(InNamespace), CreatedAt(InDate) {}
};

/** Represents a namespace */
struct FCloudNamespace
{
	FString Name;
	TArray<FString> Buckets;

	FCloudNamespace() = default;
	FCloudNamespace(const FString& InName) : Name(InName) {}
};

/**
 * Dialog for downloading builds from cloud storage
 */
class SCloudDownloadDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCloudDownloadDialog) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SWindow> InParentWindow);

	/** Returns the selected download directory */
	FString GetDownloadDirectory() const { return DownloadDirectory; }

	/** Returns true if download was initiated */
	bool WasDownloadInitiated() const { return bDownloadInitiated; }

	/** Returns true if partial downloads were used */
	bool WasPartialDownloadUsed() const { return bPartialDownloadUsed; }

	/** Returns zen.exe path */
	FString GetZenExePath() const { return ZenExePath; }

	/** Returns OidcToken.exe path */
	FString GetOidcExePath() const { return OidcExePath; }

	/** Returns cloud namespace */
	FString GetCloudNamespace() const { return SelectedBuild.IsValid() ? SelectedBuild->Namespace : FString(); }

	/** Returns cloud bucket */
	FString GetCloudBucket() const { return SelectedBuild.IsValid() ? SelectedBuild->BucketId : FString(); }

	/** Returns cloud build ID */
	FString GetCloudBuildId() const { return SelectedBuild.IsValid() ? SelectedBuild->BuildId : FString(); }

	/** Returns proxy URL */
	FString GetProxyUrl() const { return ProxyUrl; }

private:
	// Configuration reading
	bool LoadCloudSettings();

	// Zen.exe interaction
	bool QueryNamespaces();
	bool QueryBuilds(const FString& Namespace, const FString& BuildType);
	bool DownloadBuild(const FCloudBuild& Build);
	int32 RunZenCommand(const FString& Command, FString& OutResult);

	// UI callbacks
	FReply OnRefreshNamespacesClicked();
	FReply OnQueryBuildsClicked();
	FReply OnDownloadClicked();
	FReply OnCancelClicked();
	FReply OnBrowseDirectoryClicked();

	// Dropdown callbacks
	TSharedRef<SWidget> OnGenerateNamespaceWidget(TSharedPtr<FString> InItem);
	void OnNamespaceSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	FText GetCurrentNamespaceText() const;

	TSharedRef<SWidget> OnGenerateBuildTypeWidget(TSharedPtr<FString> InItem);
	void OnBuildTypeSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	FText GetCurrentBuildTypeText() const;

	TSharedRef<SWidget> OnGenerateStreamWidget(TSharedPtr<FString> InItem);
	void OnStreamSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	FText GetCurrentStreamText() const;

	TSharedRef<SWidget> OnGeneratePlatformWidget(TSharedPtr<FString> InItem);
	void OnPlatformSelectionChanged(TSharedPtr<FString> NewSelection, ESelectInfo::Type SelectInfo);
	FText GetCurrentPlatformText() const;

	TSharedRef<SWidget> OnGenerateBuildWidget(TSharedPtr<FCloudBuild> InItem);
	void OnBuildSelectionChanged(TSharedPtr<FCloudBuild> NewSelection, ESelectInfo::Type SelectInfo);
	FText GetCurrentBuildText() const;

	void OnDownloadDirectoryChanged(const FText& NewText);
	void OnBuildFilterTextChanged(const FText& NewText);

	// Helper functions for cascading dropdowns
	void UpdateAvailableStreams();
	void UpdateAvailablePlatforms(const FString& PreviousPlatform = FString());
	TArray<FString> GetMatchingBuckets() const;
	void ApplyBuildFilter();

	// Parent window
	TWeakPtr<SWindow> ParentWindow;

	// Cloud settings
	FString ZenExePath;
	FString OidcExePath;
	FString ProxyUrl;

	// UI state
	TArray<TSharedPtr<FString>> NamespaceList;
	TSharedPtr<FString> SelectedNamespace;
	TMap<FString, TArray<FString>> NamespaceBuckets; // Map of namespace -> buckets

	TArray<TSharedPtr<FString>> BuildTypeList;
	TSharedPtr<FString> SelectedBuildType;

	TArray<TSharedPtr<FString>> StreamList;
	TSharedPtr<FString> SelectedStream;

	TArray<TSharedPtr<FString>> PlatformList;
	TSharedPtr<FString> SelectedPlatform;

	TArray<TSharedPtr<FCloudBuild>> BuildList;
	TArray<TSharedPtr<FCloudBuild>> FilteredBuildList;  // Filtered list based on BuildFilterText
	TSharedPtr<FCloudBuild> SelectedBuild;
	FString BuildFilterText;  // Regex filter for build names

	FString DownloadDirectory;
	bool bDownloadInitiated = false;
	bool bPartialDownloadUsed = false;
	std::atomic<bool> bCancelRequested{false};  // Set when user clicks Cancel during download

	// UI widgets
	TSharedPtr<SComboBox<TSharedPtr<FString>>> NamespaceComboBox;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> BuildTypeComboBox;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> StreamComboBox;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> PlatformComboBox;
	TSharedPtr<SComboBox<TSharedPtr<FCloudBuild>>> BuildComboBox;
	TSharedPtr<SEditableTextBox> BuildFilterTextBox;
	TSharedPtr<SEditableTextBox> DownloadDirectoryTextBox;
	TSharedPtr<SCheckBox> UsePartialDownloadsCheckBox;
	TSharedPtr<STextBlock> StatusTextBlock;
};
