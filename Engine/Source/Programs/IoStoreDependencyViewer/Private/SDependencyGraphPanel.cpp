// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDependencyGraphPanel.h"
#include "Rendering/DrawElements.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "Misc/Paths.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/IToolTip.h"
#include "Internationalization/Regex.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"

#define LOCTEXT_NAMESPACE "SDependencyGraphPanel"

void SDependencyGraphPanel::Construct(const FArguments& InArgs)
{
	ViewOffset = FVector2D(100, 100);
	ZoomLevel = 1.0f;
	bIsPanning = false;
	LastClickTime = 0.0;
	LastClickPosition = FVector2D::ZeroVector;

	// Enable tooltips by setting a dynamic tooltip text attribute
	// Use CreateSP instead of CreateRaw to track widget lifetime (prevents dangling pointer if tooltip evaluated after widget destroyed)
	SetToolTipText(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateSP(this, &SDependencyGraphPanel::GetTooltipText)));
}

SDependencyGraphPanel::~SDependencyGraphPanel()
{
	// Clear all node references
	AllNodes.Empty();
	RootNode.Reset();
	SelectedNode.Reset();
	HoveredNode.Reset();
	PackageIdToAsset.Empty();
}

void SDependencyGraphPanel::RebuildGraph(TSharedPtr<FIoStoreAssetInfo> RootAsset,
	const TMap<FPackageId, TSharedPtr<FIoStoreAssetInfo>>& InPackageIdToAsset,
	int32 DepthLimit,
	int32 ReferencerDepthLimit)
{
	// Clear existing graph
	AllNodes.Empty();
	RootNode.Reset();
	SelectedNode.Reset();
	HoveredNode.Reset();
	RightClickedNode.Reset();
	MatchedNodes.Empty();
	NodeCanReachMatched.Empty();
	NodeReachableFromSelected.Empty();
	NodeReachableFromMatched.Empty();
	NodeCanReachSelected.Empty();
	ReverseEdges.Empty();
	PackageIdToAsset = InPackageIdToAsset;

	// Store depth limits for copy functions
	CurrentDependencyDepth = DepthLimit;
	CurrentReferencerDepth = ReferencerDepthLimit;

	if (!RootAsset.IsValid())
	{
		return;
	}

	// Create root node
	RootNode = MakeShared<FGraphNode>();
	RootNode->AssetInfo = RootAsset;
	RootNode->Position = FVector2D(0, 0);
	RootNode->Size = FVector2D(NodeWidth, NodeHeight);
	RootNode->bIsRoot = true;
	AllNodes.Add(RootNode);

	// Set root as selected by default
	SelectedNode = RootNode;

	// Track created nodes to avoid duplicates
	TMap<FPackageId, TSharedPtr<FGraphNode>> CreatedNodes;
	if (RootAsset->PackageId.IsValid())
	{
		CreatedNodes.Add(RootAsset->PackageId, RootNode);
	}

	// Build dependency tree (to the right)
	float DependencyX = HorizontalSpacing;
	float DependencyY = 0;
	CreateDependencyNodes(RootNode, RootAsset, 0, DepthLimit, CreatedNodes, DependencyX, DependencyY, true);

	// Build referencer tree (to the left) if depth > 0
	if (ReferencerDepthLimit > 0)
	{
		float ReferencerX = -HorizontalSpacing;
		float ReferencerY = 0;
		CreateReferencerNodes(RootNode, RootAsset, 0, ReferencerDepthLimit, CreatedNodes, PackageIdToReferencers, ReferencerX, ReferencerY, true);
	}

	// Center the root node among all its children (both dependencies and referencers)
	if (AllNodes.Num() > 1)
	{
		float MinY = FLT_MAX;
		float MaxY = -FLT_MAX;

		for (const TSharedPtr<FGraphNode>& Node : AllNodes)
		{
			if (Node.IsValid() && Node != RootNode)
			{
				MinY = FMath::Min(MinY, Node->Position.Y);
				MaxY = FMath::Max(MaxY, Node->Position.Y);
			}
		}

		if (MinY < FLT_MAX && MaxY > -FLT_MAX)
		{
			RootNode->Position.Y = (MinY + MaxY) * 0.5f;
		}
	}

	// Build reverse edge map for efficient referencer queries
	BuildReverseEdges();

	// Re-apply search if there's search text
	PerformSearch();
}

void SDependencyGraphPanel::CreateDependencyNodes(TSharedPtr<FGraphNode> ParentNode, TSharedPtr<FIoStoreAssetInfo> ParentAsset,
	int32 CurrentDepth, int32 MaxDepth, TMap<FPackageId, TSharedPtr<FGraphNode>>& CreatedNodes,
	float& CurrentX, float& CurrentY, bool bIsRootCall)
{
	if (CurrentDepth >= MaxDepth || !ParentAsset.IsValid())
	{
		return;
	}

	float ChildX = ParentNode->Position.X + HorizontalSpacing;
	float StartY = CurrentY;

	// Process hard dependencies
	for (const FAssetDependency& Dependency : ParentAsset->HardDependencies)
	{
		TSharedPtr<FIoStoreAssetInfo>* DepAssetPtr = PackageIdToAsset.Find(Dependency.PackageId);
		if (!DepAssetPtr || !DepAssetPtr->IsValid())
		{
			continue;
		}

		TSharedPtr<FIoStoreAssetInfo> DepAsset = *DepAssetPtr;
		TSharedPtr<FGraphNode>* ExistingNodePtr = CreatedNodes.Find(Dependency.PackageId);

		if (ExistingNodePtr)
		{
			// Node already exists, just create the connection
			ParentNode->HardDependencies.Add(*ExistingNodePtr);
		}
		else
		{
			// Create new node
			TSharedPtr<FGraphNode> DepNode = MakeShared<FGraphNode>();
			DepNode->AssetInfo = DepAsset;
			DepNode->Position = FVector2D(ChildX, CurrentY);
			DepNode->Size = FVector2D(NodeWidth, NodeHeight);

			AllNodes.Add(DepNode);
			ParentNode->HardDependencies.Add(DepNode);

			if (Dependency.PackageId.IsValid())
			{
				CreatedNodes.Add(Dependency.PackageId, DepNode);
			}

			CurrentY += VerticalSpacing;

			// Recursively create children
			float ChildCurrentX = ChildX;
			CreateDependencyNodes(DepNode, DepAsset, CurrentDepth + 1, MaxDepth, CreatedNodes, ChildCurrentX, CurrentY);
		}
	}

	// Process soft dependencies
	for (const FAssetDependency& Dependency : ParentAsset->SoftDependencies)
	{
		TSharedPtr<FIoStoreAssetInfo>* DepAssetPtr = PackageIdToAsset.Find(Dependency.PackageId);
		if (!DepAssetPtr || !DepAssetPtr->IsValid())
		{
			continue;
		}

		TSharedPtr<FIoStoreAssetInfo> DepAsset = *DepAssetPtr;
		TSharedPtr<FGraphNode>* ExistingNodePtr = CreatedNodes.Find(Dependency.PackageId);

		if (ExistingNodePtr)
		{
			// Node already exists, just create the connection
			ParentNode->SoftDependencies.Add(*ExistingNodePtr);
		}
		else
		{
			// Create new node
			TSharedPtr<FGraphNode> DepNode = MakeShared<FGraphNode>();
			DepNode->AssetInfo = DepAsset;
			DepNode->Position = FVector2D(ChildX, CurrentY);
			DepNode->Size = FVector2D(NodeWidth, NodeHeight);

			AllNodes.Add(DepNode);
			ParentNode->SoftDependencies.Add(DepNode);

			if (Dependency.PackageId.IsValid())
			{
				CreatedNodes.Add(Dependency.PackageId, DepNode);
			}

			CurrentY += VerticalSpacing;

			// Recursively create children
			float ChildCurrentX = ChildX;
			CreateDependencyNodes(DepNode, DepAsset, CurrentDepth + 1, MaxDepth, CreatedNodes, ChildCurrentX, CurrentY);
		}
	}

	// Center parent node vertically among its children (skip root node)
	if (!bIsRootCall && CurrentY > StartY)
	{
		float MidY = (StartY + CurrentY - VerticalSpacing) * 0.5f;
		ParentNode->Position.Y = MidY;
	}
}

void SDependencyGraphPanel::CreateReferencerNodes(TSharedPtr<FGraphNode> ChildNode, TSharedPtr<FIoStoreAssetInfo> ChildAsset,
	int32 CurrentDepth, int32 MaxDepth, TMap<FPackageId, TSharedPtr<FGraphNode>>& CreatedNodes,
	const TMap<FPackageId, TArray<FAssetDependency>>& ReferencerMap,
	float& CurrentX, float& CurrentY, bool bIsRootCall)
{
	if (CurrentDepth >= MaxDepth || !ChildAsset.IsValid() || !ChildAsset->PackageId.IsValid())
	{
		return;
	}

	// Find referencers of this asset
	const TArray<FAssetDependency>* Referencers = ReferencerMap.Find(ChildAsset->PackageId);
	if (!Referencers || Referencers->Num() == 0)
	{
		return;
	}

	float ParentX = ChildNode->Position.X - HorizontalSpacing;
	float StartY = CurrentY;

	// Process all referencers
	for (const FAssetDependency& Referencer : *Referencers)
	{
		TSharedPtr<FIoStoreAssetInfo>* RefAssetPtr = PackageIdToAsset.Find(Referencer.PackageId);
		if (!RefAssetPtr || !RefAssetPtr->IsValid())
		{
			continue;
		}

		TSharedPtr<FIoStoreAssetInfo> RefAsset = *RefAssetPtr;
		TSharedPtr<FGraphNode>* ExistingNodePtr = CreatedNodes.Find(Referencer.PackageId);

		if (ExistingNodePtr)
		{
			// Node already exists, just create the connection
			// The connection goes from referencer to referenced
			if (Referencer.bIsSoftReference)
			{
				(*ExistingNodePtr)->SoftDependencies.Add(ChildNode);
			}
			else
			{
				(*ExistingNodePtr)->HardDependencies.Add(ChildNode);
			}
		}
		else
		{
			// Create new node
			TSharedPtr<FGraphNode> RefNode = MakeShared<FGraphNode>();
			RefNode->AssetInfo = RefAsset;
			RefNode->Position = FVector2D(ParentX, CurrentY);
			RefNode->Size = FVector2D(NodeWidth, NodeHeight);

			AllNodes.Add(RefNode);

			// The connection goes from referencer to referenced
			if (Referencer.bIsSoftReference)
			{
				RefNode->SoftDependencies.Add(ChildNode);
			}
			else
			{
				RefNode->HardDependencies.Add(ChildNode);
			}

			if (Referencer.PackageId.IsValid())
			{
				CreatedNodes.Add(Referencer.PackageId, RefNode);
			}

			CurrentY += VerticalSpacing;

			// Recursively create parent referencers
			float ParentCurrentX = ParentX;
			CreateReferencerNodes(RefNode, RefAsset, CurrentDepth + 1, MaxDepth, CreatedNodes, ReferencerMap, ParentCurrentX, CurrentY);
		}
	}

	// Center child node vertically among its referencers (skip root node)
	if (!bIsRootCall && CurrentY > StartY)
	{
		float MidY = (StartY + CurrentY - VerticalSpacing) * 0.5f;
		ChildNode->Position.Y = MidY;
	}
}

void SDependencyGraphPanel::RebuildGraphAllAssets(const TArray<TSharedPtr<FIoStoreAssetInfo>>& Assets,
	TSharedPtr<FIoStoreAssetInfo> SelectedAsset)
{
	// Clear existing graph
	AllNodes.Empty();
	RootNode.Reset();
	SelectedNode.Reset();
	HoveredNode.Reset();
	RightClickedNode.Reset();
	MatchedNodes.Empty();
	NodeCanReachMatched.Empty();
	NodeReachableFromSelected.Empty();
	NodeReachableFromMatched.Empty();
	NodeCanReachSelected.Empty();

	if (Assets.Num() == 0)
	{
		return;
	}

	// Build a set of all asset package IDs for quick lookup
	TSet<FPackageId> AssetPackageIds;
	for (const TSharedPtr<FIoStoreAssetInfo>& Asset : Assets)
	{
		if (Asset.IsValid() && Asset->PackageId.IsValid())
		{
			AssetPackageIds.Add(Asset->PackageId);
		}
	}

	// Find root assets (assets with no dependencies to other assets in the filtered set)
	TArray<TSharedPtr<FIoStoreAssetInfo>> RootAssets;
	for (const TSharedPtr<FIoStoreAssetInfo>& Asset : Assets)
	{
		if (!Asset.IsValid())
		{
			continue;
		}

		bool bHasDependencyInSet = false;
		for (const FAssetDependency& Dep : Asset->HardDependencies)
		{
			if (AssetPackageIds.Contains(Dep.PackageId))
			{
				bHasDependencyInSet = true;
				break;
			}
		}

		if (!bHasDependencyInSet)
		{
			RootAssets.Add(Asset);
		}
	}

	// If no root assets found, just use all assets as roots
	if (RootAssets.Num() == 0)
	{
		RootAssets = Assets;
	}

	// Create nodes for all assets first
	TMap<FPackageId, TSharedPtr<FGraphNode>> CreatedNodes;
	for (const TSharedPtr<FIoStoreAssetInfo>& Asset : Assets)
	{
		if (Asset.IsValid() && Asset->PackageId.IsValid())
		{
			TSharedPtr<FGraphNode> Node = MakeShared<FGraphNode>();
			Node->AssetInfo = Asset;
			Node->Size = FVector2D(NodeWidth, NodeHeight);

			if (SelectedAsset.IsValid() && Asset == SelectedAsset)
			{
				Node->bIsRoot = true;
				SelectedNode = Node;
			}

			CreatedNodes.Add(Asset->PackageId, Node);
			AllNodes.Add(Node);
		}
	}

	// Build dependency connections
	for (const TSharedPtr<FIoStoreAssetInfo>& Asset : Assets)
	{
		if (!Asset.IsValid() || !Asset->PackageId.IsValid())
		{
			continue;
		}

		TSharedPtr<FGraphNode>* SourceNodePtr = CreatedNodes.Find(Asset->PackageId);
		if (!SourceNodePtr)
		{
			continue;
		}

		TSharedPtr<FGraphNode> SourceNode = *SourceNodePtr;

		// Connect hard dependencies
		for (const FAssetDependency& Dep : Asset->HardDependencies)
		{
			TSharedPtr<FGraphNode>* TargetNodePtr = CreatedNodes.Find(Dep.PackageId);
			if (TargetNodePtr)
			{
				SourceNode->HardDependencies.Add(*TargetNodePtr);
			}
		}

		// Connect soft dependencies
		for (const FAssetDependency& Dep : Asset->SoftDependencies)
		{
			TSharedPtr<FGraphNode>* TargetNodePtr = CreatedNodes.Find(Dep.PackageId);
			if (TargetNodePtr)
			{
				SourceNode->SoftDependencies.Add(*TargetNodePtr);
			}
		}
	}

	// Layout nodes in layers from left to right
	TSet<TSharedPtr<FGraphNode>> VisitedNodes;
	TArray<TArray<TSharedPtr<FGraphNode>>> Layers;

	// Start with root assets in layer 0
	TArray<TSharedPtr<FGraphNode>> CurrentLayer;
	for (const TSharedPtr<FIoStoreAssetInfo>& RootAsset : RootAssets)
	{
		if (RootAsset.IsValid() && RootAsset->PackageId.IsValid())
		{
			TSharedPtr<FGraphNode>* NodePtr = CreatedNodes.Find(RootAsset->PackageId);
			if (NodePtr)
			{
				CurrentLayer.Add(*NodePtr);
				VisitedNodes.Add(*NodePtr);
			}
		}
	}

	// Build layers
	while (CurrentLayer.Num() > 0)
	{
		Layers.Add(CurrentLayer);
		TArray<TSharedPtr<FGraphNode>> NextLayer;

		for (const TSharedPtr<FGraphNode>& Node : CurrentLayer)
		{
			// Add all dependencies to next layer
			for (const TWeakPtr<FGraphNode>& WeakDepNode : Node->HardDependencies)
			{
				if (TSharedPtr<FGraphNode> DepNode = WeakDepNode.Pin())
				{
					if (!VisitedNodes.Contains(DepNode))
					{
						NextLayer.AddUnique(DepNode);
						VisitedNodes.Add(DepNode);
					}
				}
			}
			for (const TWeakPtr<FGraphNode>& WeakDepNode : Node->SoftDependencies)
			{
				if (TSharedPtr<FGraphNode> DepNode = WeakDepNode.Pin())
				{
					if (!VisitedNodes.Contains(DepNode))
					{
						NextLayer.AddUnique(DepNode);
						VisitedNodes.Add(DepNode);
					}
				}
			}
		}

		CurrentLayer = NextLayer;
	}

	// Position nodes in layers
	for (int32 LayerIndex = 0; LayerIndex < Layers.Num(); ++LayerIndex)
	{
		const TArray<TSharedPtr<FGraphNode>>& Layer = Layers[LayerIndex];
		float X = LayerIndex * HorizontalSpacing;

		for (int32 NodeIndex = 0; NodeIndex < Layer.Num(); ++NodeIndex)
		{
			float Y = NodeIndex * VerticalSpacing;
			Layer[NodeIndex]->Position = FVector2D(X, Y);
		}
	}

	// Build reverse edge map for efficient referencer queries
	BuildReverseEdges();

	// Re-apply search if there's search text
	PerformSearch();
}

void SDependencyGraphPanel::RebuildGraphContainers(const TArray<TSharedPtr<FIoStoreAssetInfo>>& AllAssets)
{
	// Clear existing graph
	AllNodes.Empty();
	RootNode.Reset();
	SelectedNode.Reset();
	HoveredNode.Reset();
	RightClickedNode.Reset();
	MatchedNodes.Empty();
	NodeCanReachMatched.Empty();
	NodeReachableFromSelected.Empty();
	NodeReachableFromMatched.Empty();
	NodeCanReachSelected.Empty();

	if (AllAssets.Num() == 0)
	{
		return;
	}

	// Build a map of container -> assets
	TMap<FString, TArray<TSharedPtr<FIoStoreAssetInfo>>> ContainerToAssets;
	for (const TSharedPtr<FIoStoreAssetInfo>& Asset : AllAssets)
	{
		if (Asset.IsValid())
		{
			ContainerToAssets.FindOrAdd(Asset->ContainerName).Add(Asset);
		}
	}

	// Create a map of container name -> node for quick lookup
	TMap<FString, TSharedPtr<FGraphNode>> ContainerNodes;

	// Check if we have any valid containers (all assets might be invalid)
	// If ContainerToAssets is empty, NumContainers would be 0 and cause division by zero in angle calculation
	int32 NumContainers = ContainerToAssets.Num();
	if (NumContainers == 0)
	{
		return; // No valid containers to display
	}

	// Create nodes for each container in a circular layout
	int32 Index = 0;
	float Radius = FMath::Max(400.0f, NumContainers * 50.0f); // Radius increases with container count
	FVector2D Center(0, 0);

	for (const auto& Pair : ContainerToAssets)
	{
		TSharedPtr<FGraphNode> Node = MakeShared<FGraphNode>();
		Node->bIsContainer = true;
		Node->ContainerName = Pair.Key;

		// Arrange in a circle
		float Angle = (2.0f * PI * Index) / NumContainers;
		Node->Position = Center + FVector2D(FMath::Cos(Angle), FMath::Sin(Angle)) * Radius;
		Node->Size = FVector2D(NodeWidth, NodeHeight);

		AllNodes.Add(Node);
		ContainerNodes.Add(Pair.Key, Node);
		Index++;
	}

	// Build dependency counts between containers
	for (const auto& Pair : ContainerToAssets)
	{
		const FString& SourceContainer = Pair.Key;
		const TArray<TSharedPtr<FIoStoreAssetInfo>>& SourceAssets = Pair.Value;

		TSharedPtr<FGraphNode>* SourceNodePtr = ContainerNodes.Find(SourceContainer);
		if (!SourceNodePtr)
		{
			continue;
		}

		TSharedPtr<FGraphNode> SourceNode = *SourceNodePtr;

		// Count dependencies to other containers
		TMap<FString, int32> HardDepsToContainer;
		TMap<FString, int32> SoftDepsToContainer;

		for (const TSharedPtr<FIoStoreAssetInfo>& SourceAsset : SourceAssets)
		{
			// Count hard dependencies
			for (const FAssetDependency& Dep : SourceAsset->HardDependencies)
			{
				TSharedPtr<FIoStoreAssetInfo>* TargetAssetPtr = PackageIdToAsset.Find(Dep.PackageId);
				if (TargetAssetPtr && TargetAssetPtr->IsValid())
				{
					const FString& TargetContainer = (*TargetAssetPtr)->ContainerName;
					if (TargetContainer != SourceContainer)
					{
						HardDepsToContainer.FindOrAdd(TargetContainer)++;
					}
				}
			}

			// Count soft dependencies
			for (const FAssetDependency& Dep : SourceAsset->SoftDependencies)
			{
				TSharedPtr<FIoStoreAssetInfo>* TargetAssetPtr = PackageIdToAsset.Find(Dep.PackageId);
				if (TargetAssetPtr && TargetAssetPtr->IsValid())
				{
					const FString& TargetContainer = (*TargetAssetPtr)->ContainerName;
					if (TargetContainer != SourceContainer)
					{
						SoftDepsToContainer.FindOrAdd(TargetContainer)++;
					}
				}
			}
		}

		// Create connections to other container nodes
		for (const auto& DepPair : HardDepsToContainer)
		{
			TSharedPtr<FGraphNode>* TargetNodePtr = ContainerNodes.Find(DepPair.Key);
			if (TargetNodePtr)
			{
				// Store hard dependency connection
				TSharedPtr<FGraphNode> TargetNode = *TargetNodePtr;
				// Store count on source node (edge property) to prevent overwriting when multiple sources point to same target
				// Using parallel arrays: count at index i corresponds to dependency at index i
				SourceNode->HardDependencies.Add(TargetNode);
				SourceNode->HardDependencyCounts.Add(DepPair.Value);
			}
		}

		for (const auto& DepPair : SoftDepsToContainer)
		{
			TSharedPtr<FGraphNode>* TargetNodePtr = ContainerNodes.Find(DepPair.Key);
			if (TargetNodePtr)
			{
				// Store soft dependency connection
				TSharedPtr<FGraphNode> TargetNode = *TargetNodePtr;

				// Only add if not already in hard dependencies
				if (!SourceNode->HardDependencies.Contains(TargetNode))
				{
					// Store count on source node (edge property) to prevent overwriting when multiple sources point to same target
					// Using parallel arrays: count at index i corresponds to dependency at index i
					SourceNode->SoftDependencies.Add(TargetNode);
					SourceNode->SoftDependencyCounts.Add(DepPair.Value);
				}
			}
		}
	}

	// Build reverse edge map for efficient referencer queries
	BuildReverseEdges();

	// Re-apply search if there's search text
	PerformSearch();
}

void SDependencyGraphPanel::ZoomToFit()
{
	if (AllNodes.Num() == 0)
	{
		return;
	}

	// Find bounds
	FVector2D MinBounds(FLT_MAX, FLT_MAX);
	FVector2D MaxBounds(-FLT_MAX, -FLT_MAX);

	for (const TSharedPtr<FGraphNode>& Node : AllNodes)
	{
		if (!Node.IsValid())
		{
			continue; // Skip invalid nodes
		}

		MinBounds.X = FMath::Min(MinBounds.X, Node->Position.X);
		MinBounds.Y = FMath::Min(MinBounds.Y, Node->Position.Y);
		MaxBounds.X = FMath::Max(MaxBounds.X, Node->Position.X + Node->Size.X);
		MaxBounds.Y = FMath::Max(MaxBounds.Y, Node->Position.Y + Node->Size.Y);
	}

	FVector2D GraphSize = MaxBounds - MinBounds;
	FVector2D ViewSize = GetTickSpaceGeometry().GetLocalSize();

	// Calculate zoom to fit, guarding against division by zero when nodes are aligned on an axis
	constexpr float MinGraphSize = 1.0f; // Minimum size to avoid division by zero
	float ZoomX = GraphSize.X > MinGraphSize ? (ViewSize.X - 100) / GraphSize.X : 1.0f;
	float ZoomY = GraphSize.Y > MinGraphSize ? (ViewSize.Y - 100) / GraphSize.Y : 1.0f;
	ZoomLevel = FMath::Min(ZoomX, ZoomY);
	ZoomLevel = FMath::Clamp(ZoomLevel, 0.1f, 2.0f);

	// Center the view
	ViewOffset = (ViewSize * 0.5f) - ((MinBounds + GraphSize * 0.5f) * ZoomLevel);
}

FVector2D SDependencyGraphPanel::GetNodeScreenPosition(TSharedPtr<FGraphNode> Node) const
{
	return Node->Position * ZoomLevel + ViewOffset;
}

int32 SDependencyGraphPanel::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// Enable clipping to prevent drawing outside this panel
	OutDrawElements.PushClip(FSlateClippingZone(AllottedGeometry));

	// Draw background
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		FLinearColor(0.02f, 0.02f, 0.02f)
	);

	// Draw connections first (below nodes)
	for (const TSharedPtr<FGraphNode>& Node : AllNodes)
	{
		if (!Node.IsValid())
		{
			continue; // Skip invalid nodes (defensive check matching later rendering code)
		}

		FVector2D StartPos = GetNodeScreenPosition(Node);
		StartPos += Node->Size * ZoomLevel * 0.5f; // Center of node

		// Draw hard dependencies
		for (int32 DepIndex = 0; DepIndex < Node->HardDependencies.Num(); ++DepIndex)
		{
			TSharedPtr<FGraphNode> DepNode = Node->HardDependencies[DepIndex].Pin();
			if (!DepNode.IsValid())
			{
				continue;
			}

			FVector2D EndPos = GetNodeScreenPosition(DepNode);
			EndPos += DepNode->Size * ZoomLevel * 0.5f;

			// Determine line color - Blue if this edge is on a path from selected to matched
			FLinearColor LineColor = FLinearColor(0.0f, 0.8f, 0.0f); // Default green

			// Highlight blue if this edge is part of a path
			bool bShouldHighlight = false;

			// Determine which side of the tree we're on
			if (SelectedNode.IsValid())
			{
				bool bIsRightSide = Node->Position.X >= SelectedNode->Position.X;

				if (bIsRightSide)
				{
					// Right side (dependencies): Check dependency path only
					bShouldHighlight = NodeReachableFromSelected.FindRef(Node) && NodeCanReachMatched.FindRef(DepNode);
				}
				else
				{
					// Left side (referencers): Check referencer path only
					// Both nodes must be reachable from a matched node
					bShouldHighlight = NodeReachableFromMatched.FindRef(Node) && NodeReachableFromMatched.FindRef(DepNode);
				}
			}

			if (bShouldHighlight)
			{
				LineColor = FLinearColor(0.0f, 0.4f, 0.9f); // Blue for path edges
			}

			DrawConnection(AllottedGeometry, OutDrawElements, LayerId, StartPos, EndPos,
				LineColor, 2.0f);

			// For container nodes, draw dependency count on the line (from parallel array)
			int32 HardCount = (DepIndex < Node->HardDependencyCounts.Num()) ? Node->HardDependencyCounts[DepIndex] : 0;
			if (Node->bIsContainer && HardCount > 0)
			{
				FVector2D MidPoint = (StartPos + EndPos) * 0.5f;
				FSlateFontInfo FontInfo = FAppStyle::GetFontStyle("NormalFont");
				FontInfo.Size = FMath::Max(8.0f, 9.0f * ZoomLevel);

				FString CountText = FString::Printf(TEXT("H:%d"), HardCount);

				FSlateDrawElement::MakeText(
					OutDrawElements,
					LayerId + 1,
					AllottedGeometry.ToPaintGeometry(FVector2f::ZeroVector, FSlateLayoutTransform(FVector2f(MidPoint))),
					CountText,
					FontInfo,
					ESlateDrawEffect::None,
					FLinearColor(0.0f, 1.0f, 0.0f)
				);
			}
		}

		// Draw soft dependencies
		for (int32 DepIndex = 0; DepIndex < Node->SoftDependencies.Num(); ++DepIndex)
		{
			TSharedPtr<FGraphNode> DepNode = Node->SoftDependencies[DepIndex].Pin();
			if (!DepNode.IsValid())
			{
				continue;
			}

			FVector2D EndPos = GetNodeScreenPosition(DepNode);
			EndPos += DepNode->Size * ZoomLevel * 0.5f;

			// Determine line color - Blue if this edge is on a path from selected to matched
			FLinearColor LineColor = FLinearColor(0.9f, 0.6f, 0.0f); // Default orange

			// Highlight blue if this edge is part of a path
			bool bShouldHighlight = false;

			// Determine which side of the tree we're on
			if (SelectedNode.IsValid())
			{
				bool bIsRightSide = Node->Position.X >= SelectedNode->Position.X;

				if (bIsRightSide)
				{
					// Right side (dependencies): Check dependency path only
					bShouldHighlight = NodeReachableFromSelected.FindRef(Node) && NodeCanReachMatched.FindRef(DepNode);
				}
				else
				{
					// Left side (referencers): Check referencer path only
					// Both nodes must be reachable from a matched node
					bShouldHighlight = NodeReachableFromMatched.FindRef(Node) && NodeReachableFromMatched.FindRef(DepNode);
				}
			}

			if (bShouldHighlight)
			{
				LineColor = FLinearColor(0.0f, 0.4f, 0.9f); // Blue for path edges
			}

			DrawConnection(AllottedGeometry, OutDrawElements, LayerId, StartPos, EndPos,
				LineColor, 1.5f);

			// For container nodes, draw dependency count on the line (from parallel array)
			int32 SoftCount = (DepIndex < Node->SoftDependencyCounts.Num()) ? Node->SoftDependencyCounts[DepIndex] : 0;
			if (Node->bIsContainer && SoftCount > 0)
			{
				FVector2D MidPoint = (StartPos + EndPos) * 0.5f;
				FSlateFontInfo FontInfo = FAppStyle::GetFontStyle("NormalFont");
				FontInfo.Size = FMath::Max(8.0f, 9.0f * ZoomLevel);

				FString CountText = FString::Printf(TEXT("S:%d"), SoftCount);

				FSlateDrawElement::MakeText(
					OutDrawElements,
					LayerId + 1,
					AllottedGeometry.ToPaintGeometry(FVector2f::ZeroVector, FSlateLayoutTransform(FVector2f(MidPoint))),
					CountText,
					FontInfo,
					ESlateDrawEffect::None,
					FLinearColor(0.9f, 0.6f, 0.0f)
				);
			}
		}
	}

	// Draw nodes
	for (const TSharedPtr<FGraphNode>& Node : AllNodes)
	{
		DrawNode(AllottedGeometry, OutDrawElements, LayerId, Node);
	}

	// Pop the clipping zone
	OutDrawElements.PopClip();

	return LayerId;
}

void SDependencyGraphPanel::DrawNode(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32& LayerId,
	TSharedPtr<FGraphNode> Node) const
{
	if (!Node.IsValid())
	{
		return;
	}

	// Handle container nodes differently
	if (Node->bIsContainer)
	{
		FVector2D ScreenPos = GetNodeScreenPosition(Node);
		FVector2D ScreenSize = Node->Size * ZoomLevel;

		// Container nodes are blue
		FLinearColor NodeColor = FLinearColor(0.2f, 0.4f, 0.8f);

		// Draw node background
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId++,
			AllottedGeometry.ToPaintGeometry(FVector2f(ScreenSize), FSlateLayoutTransform(FVector2f(ScreenPos))),
			FAppStyle::GetBrush("WhiteBrush"),
			ESlateDrawEffect::None,
			NodeColor
		);

		// Draw border
		TArray<FVector2D> BorderPoints;
		BorderPoints.Add(ScreenPos);
		BorderPoints.Add(ScreenPos + FVector2D(ScreenSize.X, 0));
		BorderPoints.Add(ScreenPos + ScreenSize);
		BorderPoints.Add(ScreenPos + FVector2D(0, ScreenSize.Y));
		BorderPoints.Add(ScreenPos);

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId++,
			AllottedGeometry.ToPaintGeometry(),
			BorderPoints,
			ESlateDrawEffect::None,
			FLinearColor(0.3f, 0.3f, 0.3f),
			true,
			2.0f
		);

		// Draw container name
		FSlateFontInfo FontInfo = FAppStyle::GetFontStyle("NormalFont");
		FontInfo.Size = FMath::Max(8.0f, 10.0f * ZoomLevel);

		FString NodeText = Node->ContainerName;
		if (NodeText.Len() > 25)
		{
			NodeText = NodeText.Left(22) + TEXT("...");
		}

		FVector2D TextPos = ScreenPos + FVector2D(5 * ZoomLevel, 5 * ZoomLevel);
		FSlateDrawElement::MakeText(
			OutDrawElements,
			LayerId++,
			AllottedGeometry.ToPaintGeometry(FVector2f(ScreenSize), FSlateLayoutTransform(FVector2f(TextPos))),
			NodeText,
			FontInfo,
			ESlateDrawEffect::None,
			FLinearColor::White
		);

		return;
	}

	// Regular asset node
	if (!Node->AssetInfo.IsValid())
	{
		return;
	}

	FVector2D ScreenPos = GetNodeScreenPosition(Node);
	FVector2D ScreenSize = Node->Size * ZoomLevel;

	// Determine node color - Green for selected, Blue for matched, Red for all others
	FLinearColor NodeColor;
	if (Node == SelectedNode)
	{
		NodeColor = FLinearColor(0.0f, 0.7f, 0.0f); // Green for selected
	}
	else if (MatchedNodes.Contains(Node))
	{
		NodeColor = FLinearColor(0.0f, 0.4f, 0.9f); // Blue for search matches
	}
	else
	{
		NodeColor = FLinearColor(0.7f, 0.0f, 0.0f); // Red for all other nodes
	}

	// Draw node background
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(FVector2f(ScreenSize), FSlateLayoutTransform(FVector2f(ScreenPos))),
		FAppStyle::GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		NodeColor
	);

	// Draw node border as an outline
	TArray<FVector2D> BorderPoints;
	BorderPoints.Add(ScreenPos); // Top-left
	BorderPoints.Add(ScreenPos + FVector2D(ScreenSize.X, 0)); // Top-right
	BorderPoints.Add(ScreenPos + ScreenSize); // Bottom-right
	BorderPoints.Add(ScreenPos + FVector2D(0, ScreenSize.Y)); // Bottom-left
	BorderPoints.Add(ScreenPos); // Close the loop

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(),
		BorderPoints,
		ESlateDrawEffect::None,
		FLinearColor(0.2f, 0.2f, 0.2f), // Dark gray border
		true,
		2.0f // Border thickness
	);

	// Draw node text - Extract just the filename without path
	FSlateFontInfo FontInfo = FAppStyle::GetFontStyle("NormalFont");
	FontInfo.Size = FMath::Max(8.0f, 10.0f * ZoomLevel);

	// Extract just the filename from the full path
	FString FullPath = Node->AssetInfo->FileName;
	FString NodeText = FPaths::GetCleanFilename(FullPath);

	// Remove extension if present
	if (NodeText.EndsWith(TEXT(".uasset")) || NodeText.EndsWith(TEXT(".umap")))
	{
		NodeText = FPaths::GetBaseFilename(NodeText);
	}

	// Truncate if still too long
	if (NodeText.Len() > 25)
	{
		NodeText = NodeText.Left(22) + TEXT("...");
	}

	FVector2D TextPos = ScreenPos + FVector2D(5 * ZoomLevel, 5 * ZoomLevel);
	FSlateDrawElement::MakeText(
		OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(FVector2f(ScreenSize), FSlateLayoutTransform(FVector2f(TextPos))),
		NodeText,
		FontInfo,
		ESlateDrawEffect::None,
		FLinearColor::White
	);

	// Draw dependency count
	FString DepText = FString::Printf(TEXT("H:%d S:%d"),
		Node->AssetInfo->HardDependencies.Num(),
		Node->AssetInfo->SoftDependencies.Num());

	// ScreenSize already includes ZoomLevel (computed as Node->Size * ZoomLevel)
	// Apply ZoomLevel only to the constant offset, not to ScreenSize.Y again
	FVector2D DepTextPos = ScreenPos + FVector2D(5 * ZoomLevel, ScreenSize.Y - 20 * ZoomLevel);
	FSlateDrawElement::MakeText(
		OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(FVector2f(ScreenSize), FSlateLayoutTransform(FVector2f(DepTextPos))),
		DepText,
		FontInfo,
		ESlateDrawEffect::None,
		FLinearColor(0.7f, 0.7f, 0.7f)
	);
}

void SDependencyGraphPanel::DrawConnection(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32& LayerId,
	const FVector2D& Start, const FVector2D& End, const FLinearColor& Color, float Thickness) const
{
	TArray<FVector2D> LinePoints;
	LinePoints.Add(Start);
	LinePoints.Add(End);

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		LayerId++,
		AllottedGeometry.ToPaintGeometry(),
		LinePoints,
		ESlateDrawEffect::None,
		Color,
		true,
		Thickness * ZoomLevel
	);
}

FReply SDependencyGraphPanel::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FVector2D LocalMousePos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		TSharedPtr<FGraphNode> ClickedNode = FindNodeAtPosition(LocalMousePos, MyGeometry);

		if (ClickedNode.IsValid())
		{
			// Check for double-click
			double CurrentTime = FPlatformTime::Seconds();
			bool bIsDoubleClick = (CurrentTime - LastClickTime < 0.3) &&
			                     ((LocalMousePos - LastClickPosition).SizeSquared() < 25.0f);

			LastClickTime = CurrentTime;
			LastClickPosition = LocalMousePos;

			if (bIsDoubleClick)
			{
				// Double-click: select this node
				SelectedNode = ClickedNode;
				return FReply::Handled();
			}
		}

		return FReply::Unhandled();
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton)
	{
		bIsPanning = true;
		LastMousePosition = MouseEvent.GetScreenSpacePosition();
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}
	else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		// Check if we right-clicked on a node
		FVector2D LocalMousePos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		RightClickedNode = FindNodeAtPosition(LocalMousePos, MyGeometry);

		// Show context menu
		TSharedPtr<SWidget> MenuContent = OnGraphContextMenuOpening();
		if (MenuContent.IsValid())
		{
			FSlateApplication::Get().PushMenu(
				AsShared(),
				FWidgetPath(),
				MenuContent.ToSharedRef(),
				MouseEvent.GetScreenSpacePosition(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu)
			);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FReply SDependencyGraphPanel::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bIsPanning)
	{
		bIsPanning = false;
		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

FReply SDependencyGraphPanel::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bIsPanning)
	{
		FVector2D CurrentMousePosition = MouseEvent.GetScreenSpacePosition();
		FVector2D Delta = CurrentMousePosition - LastMousePosition;
		ViewOffset += Delta;
		LastMousePosition = CurrentMousePosition;
		return FReply::Handled();
	}
	else
	{
		// Update hovered node for tooltip
		FVector2D LocalMousePos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		HoveredNode = FindNodeAtPosition(LocalMousePos, MyGeometry);
	}

	return FReply::Unhandled();
}

FReply SDependencyGraphPanel::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	float WheelDelta = MouseEvent.GetWheelDelta();
	float NewZoom = ZoomLevel * (1.0f + WheelDelta * 0.1f);
	NewZoom = FMath::Clamp(NewZoom, 0.1f, 3.0f);

	// Zoom towards mouse position
	FVector2D MousePos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	FVector2D WorldPosBeforeZoom = (MousePos - ViewOffset) / ZoomLevel;
	ZoomLevel = NewZoom;
	ViewOffset = MousePos - (WorldPosBeforeZoom * ZoomLevel);

	return FReply::Handled();
}

TSharedPtr<FGraphNode> SDependencyGraphPanel::FindNodeAtPosition(const FVector2D& ScreenPosition, const FGeometry& MyGeometry) const
{
	for (const TSharedPtr<FGraphNode>& Node : AllNodes)
	{
		if (!Node.IsValid())
		{
			continue;
		}

		FVector2D NodeScreenPos = GetNodeScreenPosition(Node);
		FVector2D NodeScreenSize = Node->Size * ZoomLevel;

		// Check if point is inside node bounds
		if (ScreenPosition.X >= NodeScreenPos.X && ScreenPosition.X <= NodeScreenPos.X + NodeScreenSize.X &&
		    ScreenPosition.Y >= NodeScreenPos.Y && ScreenPosition.Y <= NodeScreenPos.Y + NodeScreenSize.Y)
		{
			return Node;
		}
	}

	return nullptr;
}

FText SDependencyGraphPanel::GetTooltipText() const
{
	// Return empty if no hovered node
	if (!HoveredNode.IsValid())
	{
		return FText::GetEmpty();
	}

	FString TooltipText;

	// Handle container nodes
	if (HoveredNode->bIsContainer)
	{
		// Count total dependencies from this container (using parallel arrays)
		int32 TotalHardDeps = 0;
		int32 TotalSoftDeps = 0;
		for (int32 DepIndex = 0; DepIndex < HoveredNode->HardDependencies.Num(); ++DepIndex)
		{
			if (HoveredNode->HardDependencies[DepIndex].Pin().IsValid())
			{
				if (DepIndex < HoveredNode->HardDependencyCounts.Num())
				{
					TotalHardDeps += HoveredNode->HardDependencyCounts[DepIndex];
				}
			}
		}
		for (int32 DepIndex = 0; DepIndex < HoveredNode->SoftDependencies.Num(); ++DepIndex)
		{
			if (HoveredNode->SoftDependencies[DepIndex].Pin().IsValid())
			{
				if (DepIndex < HoveredNode->SoftDependencyCounts.Num())
				{
					TotalSoftDeps += HoveredNode->SoftDependencyCounts[DepIndex];
				}
			}
		}

		TooltipText = FString::Printf(
			TEXT("Container: %s\n")
			TEXT("Total Hard Dependencies: %d\n")
			TEXT("Total Soft Dependencies: %d"),
			*HoveredNode->ContainerName,
			TotalHardDeps,
			TotalSoftDeps
		);
	}
	// Handle regular asset nodes
	else if (HoveredNode->AssetInfo.IsValid())
	{
		const TSharedPtr<FIoStoreAssetInfo>& Asset = HoveredNode->AssetInfo;

		// Convert ChunkId to string
		FString ChunkIdStr;
		Asset->ChunkId.ToString(ChunkIdStr);

		// Convert PackageId to string
		FString PackageIdStr = Asset->PackageId.IsValid() ?
			FString::Printf(TEXT("%016llX"), Asset->PackageId.Value()) :
			TEXT("Invalid");

		TooltipText = FString::Printf(
			TEXT("Asset: %s\n")
			TEXT("Chunk ID: %s\n")
			TEXT("Package ID: %s\n")
			TEXT("Container: %s\n")
			TEXT("Size: %.2f MB\n")
			TEXT("Compressed: %.2f MB\n")
			TEXT("Hard Dependencies: %d\n")
			TEXT("Soft Dependencies: %d"),
			*Asset->FileName,
			*ChunkIdStr,
			*PackageIdStr,
			*Asset->ContainerName,
			Asset->Size / (1024.0f * 1024.0f),
			Asset->CompressedSize / (1024.0f * 1024.0f),
			Asset->HardDependencies.Num(),
			Asset->SoftDependencies.Num()
		);
	}
	else
	{
		return FText::GetEmpty();
	}

	return FText::FromString(TooltipText);
}

void SDependencyGraphPanel::OnGraphSearchTextChanged(const FText& InText)
{
	GraphSearchText = InText;
	PerformSearch();
}

void SDependencyGraphPanel::PerformSearch()
{
	MatchedNodes.Empty();

	FString SearchPattern = GraphSearchText.ToString();
	if (SearchPattern.IsEmpty())
	{
		return;
	}

	// Try to compile the regex pattern
	FRegexPattern Pattern(SearchPattern, ERegexPatternFlags::CaseInsensitive);

	for (const TSharedPtr<FGraphNode>& Node : AllNodes)
	{
		if (!Node.IsValid())
		{
			continue;
		}

		bool bMatches = false;

		// For container nodes, match against container name
		if (Node->bIsContainer)
		{
			FRegexMatcher Matcher(Pattern, Node->ContainerName);
			if (Matcher.FindNext())
			{
				bMatches = true;
			}
		}
		// For asset nodes, match against filename, package name, or container name
		else if (Node->AssetInfo.IsValid())
		{
			// Match against filename
			FRegexMatcher FilenameMatcher(Pattern, Node->AssetInfo->FileName);
			if (FilenameMatcher.FindNext())
			{
				bMatches = true;
			}

			// Match against package name
			if (!bMatches && !Node->AssetInfo->PackageName.IsEmpty())
			{
				FRegexMatcher PackageMatcher(Pattern, Node->AssetInfo->PackageName);
				if (PackageMatcher.FindNext())
				{
					bMatches = true;
				}
			}

			// Match against container name
			if (!bMatches && !Node->AssetInfo->ContainerName.IsEmpty())
			{
				FRegexMatcher ContainerMatcher(Pattern, Node->AssetInfo->ContainerName);
				if (ContainerMatcher.FindNext())
				{
					bMatches = true;
				}
			}
		}

		if (bMatches)
		{
			MatchedNodes.Add(Node);
		}
	}

	// Compute paths from selected node to matched nodes
	ComputePathsToMatchedNodes();
}

void SDependencyGraphPanel::BuildReverseEdges() const
{
	ReverseEdges.Empty();

	// Build reverse edge map for efficient incoming-edge queries (avoids O(N) scans per node)
	for (const TSharedPtr<FGraphNode>& Node : AllNodes)
	{
		if (Node.IsValid())
		{
			// For each dependency of this node, add this node to its reverse edge list
			for (const TWeakPtr<FGraphNode>& WeakDep : Node->HardDependencies)
			{
				if (TSharedPtr<FGraphNode> Dep = WeakDep.Pin())
				{
					ReverseEdges.FindOrAdd(Dep).Add(Node);
				}
			}
			for (const TWeakPtr<FGraphNode>& WeakDep : Node->SoftDependencies)
			{
				if (TSharedPtr<FGraphNode> Dep = WeakDep.Pin())
				{
					ReverseEdges.FindOrAdd(Dep).Add(Node);
				}
			}
		}
	}
}

void SDependencyGraphPanel::ComputePathsToMatchedNodes()
{
	// Clear caches
	NodeCanReachMatched.Empty();
	NodeReachableFromSelected.Empty();
	NodeReachableFromMatched.Empty();
	NodeCanReachSelected.Empty();

	// No paths to compute if no selected node or no matched nodes
	if (!SelectedNode.IsValid() || MatchedNodes.Num() == 0)
	{
		return;
	}

	// Build reverse edges for efficient path computation
	BuildReverseEdges();

	// Pre-compute all reachability for both dependency and referencer trees
	for (const TSharedPtr<FGraphNode>& Node : AllNodes)
	{
		if (Node.IsValid())
		{
			TSet<TSharedPtr<FGraphNode>> Visited;

			// For dependency tree: Selected → Node → Matched
			CanReachMatchedNode(Node, Visited);
			Visited.Empty();
			IsReachableFromSelected(Node, Visited);

			// For referencer tree: Matched → Node → Selected
			Visited.Empty();
			IsReachableFromMatched(Node, Visited);
			Visited.Empty();
			CanReachSelected(Node, Visited);
		}
	}
}

bool SDependencyGraphPanel::IsReachableFromSelected(TSharedPtr<FGraphNode> Node, TSet<TSharedPtr<FGraphNode>>& Visited) const
{
	if (!Node.IsValid())
	{
		return false;
	}

	// Check cache
	if (NodeReachableFromSelected.Contains(Node))
	{
		return NodeReachableFromSelected[Node];
	}

	// Selected node is always reachable from itself
	if (Node == SelectedNode)
	{
		NodeReachableFromSelected.Add(Node, true);
		return true;
	}

	// Prevent infinite recursion
	if (Visited.Contains(Node))
	{
		return false;
	}
	Visited.Add(Node);

	bool bReachable = false;

	// Forward direction: Check if any node that points to this node is reachable from selected
	// This handles the dependency tree (Selected -> Dependency)
	for (const TSharedPtr<FGraphNode>& OtherNode : AllNodes)
	{
		if (!OtherNode.IsValid())
		{
			continue;
		}

		// Check if OtherNode has this Node as a dependency
		bool bOtherPointsToThis = OtherNode->HardDependencies.Contains(Node) ||
		                          OtherNode->SoftDependencies.Contains(Node);

		if (bOtherPointsToThis && IsReachableFromSelected(OtherNode, Visited))
		{
			bReachable = true;
			break;
		}
	}

	// Backward direction: Check if this node points to selected (directly or indirectly)
	// This handles the referencer tree (Referencer -> Selected)
	if (!bReachable)
	{
		TSet<TSharedPtr<FGraphNode>> BackwardVisited;
		if (CanReachNode(Node, SelectedNode, BackwardVisited))
		{
			bReachable = true;
		}
	}

	NodeReachableFromSelected.Add(Node, bReachable);
	return bReachable;
}

bool SDependencyGraphPanel::CanReachMatchedNode(TSharedPtr<FGraphNode> Node, TSet<TSharedPtr<FGraphNode>>& Visited) const
{
	if (!Node.IsValid())
	{
		return false;
	}

	// Check cache
	if (NodeCanReachMatched.Contains(Node))
	{
		return NodeCanReachMatched[Node];
	}

	// If this node is a matched node, it can reach a matched node (itself)
	if (MatchedNodes.Contains(Node))
	{
		NodeCanReachMatched.Add(Node, true);
		return true;
	}

	// Prevent infinite recursion
	if (Visited.Contains(Node))
	{
		return false;
	}
	Visited.Add(Node);

	// Check if any hard dependency can reach a matched node
	bool bCanReach = false;
	for (const TWeakPtr<FGraphNode>& WeakDep : Node->HardDependencies)
	{
		if (TSharedPtr<FGraphNode> Dep = WeakDep.Pin())
		{
			if (CanReachMatchedNode(Dep, Visited))
			{
				bCanReach = true;
				break;
			}
		}
	}

	// Check soft dependencies if no path found through hard dependencies
	if (!bCanReach)
	{
		for (const TWeakPtr<FGraphNode>& WeakDep : Node->SoftDependencies)
		{
			if (TSharedPtr<FGraphNode> Dep = WeakDep.Pin())
			{
				if (CanReachMatchedNode(Dep, Visited))
				{
					bCanReach = true;
					break;
				}
			}
		}
	}

	NodeCanReachMatched.Add(Node, bCanReach);
	return bCanReach;
}

bool SDependencyGraphPanel::CanReachNode(TSharedPtr<FGraphNode> FromNode, TSharedPtr<FGraphNode> ToNode, TSet<TSharedPtr<FGraphNode>>& Visited) const
{
	if (!FromNode.IsValid() || !ToNode.IsValid())
	{
		return false;
	}

	// Found the target
	if (FromNode == ToNode)
	{
		return true;
	}

	// Prevent infinite recursion
	if (Visited.Contains(FromNode))
	{
		return false;
	}
	Visited.Add(FromNode);

	// Check if any hard dependency can reach the target
	for (const TWeakPtr<FGraphNode>& WeakDep : FromNode->HardDependencies)
	{
		if (TSharedPtr<FGraphNode> Dep = WeakDep.Pin())
		{
			if (CanReachNode(Dep, ToNode, Visited))
			{
				return true;
			}
		}
	}

	// Check soft dependencies
	for (const TWeakPtr<FGraphNode>& WeakDep : FromNode->SoftDependencies)
	{
		if (TSharedPtr<FGraphNode> Dep = WeakDep.Pin())
		{
			if (CanReachNode(Dep, ToNode, Visited))
			{
				return true;
			}
		}
	}

	return false;
}

bool SDependencyGraphPanel::IsReachableFromMatched(TSharedPtr<FGraphNode> Node, TSet<TSharedPtr<FGraphNode>>& Visited) const
{
	if (!Node.IsValid())
	{
		return false;
	}

	// Check cache
	if (NodeReachableFromMatched.Contains(Node))
	{
		return NodeReachableFromMatched[Node];
	}

	// Matched nodes are reachable from themselves
	if (MatchedNodes.Contains(Node))
	{
		NodeReachableFromMatched.Add(Node, true);
		return true;
	}

	// Prevent infinite recursion
	if (Visited.Contains(Node))
	{
		return false;
	}
	Visited.Add(Node);

	// Check if any node that points to this node is reachable from a matched node
	// Use reverse edge map for O(1) incoming edge lookup instead of O(N) AllNodes scan
	bool bReachable = false;
	if (const TArray<TSharedPtr<FGraphNode>>* IncomingNodes = ReverseEdges.Find(Node))
	{
		for (const TSharedPtr<FGraphNode>& OtherNode : *IncomingNodes)
		{
			if (OtherNode.IsValid() && IsReachableFromMatched(OtherNode, Visited))
			{
				bReachable = true;
				break;
			}
		}
	}

	NodeReachableFromMatched.Add(Node, bReachable);
	return bReachable;
}

bool SDependencyGraphPanel::CanReachSelected(TSharedPtr<FGraphNode> Node, TSet<TSharedPtr<FGraphNode>>& Visited) const
{
	if (!Node.IsValid())
	{
		return false;
	}

	// Check cache
	if (NodeCanReachSelected.Contains(Node))
	{
		return NodeCanReachSelected[Node];
	}

	// If this node is the selected node, it can reach itself
	if (Node == SelectedNode)
	{
		NodeCanReachSelected.Add(Node, true);
		return true;
	}

	// Prevent infinite recursion
	if (Visited.Contains(Node))
	{
		return false;
	}
	Visited.Add(Node);

	// Check if any hard dependency can reach the selected node
	bool bCanReach = false;
	for (const TWeakPtr<FGraphNode>& WeakDep : Node->HardDependencies)
	{
		if (TSharedPtr<FGraphNode> Dep = WeakDep.Pin())
		{
			if (CanReachSelected(Dep, Visited))
			{
				bCanReach = true;
				break;
			}
		}
	}

	// Check soft dependencies if no path found through hard dependencies
	if (!bCanReach)
	{
		for (const TWeakPtr<FGraphNode>& WeakDep : Node->SoftDependencies)
		{
			if (TSharedPtr<FGraphNode> Dep = WeakDep.Pin())
			{
				if (CanReachSelected(Dep, Visited))
				{
					bCanReach = true;
					break;
				}
			}
		}
	}

	NodeCanReachSelected.Add(Node, bCanReach);
	return bCanReach;
}

TSharedPtr<SWidget> SDependencyGraphPanel::OnGraphContextMenuOpening()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	// If we right-clicked on a node, show "Copy Asset" option first
	if (RightClickedNode.IsValid())
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyAsset", "Copy Asset"),
			LOCTEXT("CopyAssetTooltip", "Copy this asset's information to clipboard"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SDependencyGraphPanel::CopyNodeAssetToClipboard))
		);

		MenuBuilder.AddSeparator();
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CopyEntireGraph", "Copy Entire Tree"),
		LOCTEXT("CopyEntireGraphTooltip", "Copy the entire dependency tree to clipboard"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SDependencyGraphPanel::CopyEntireGraphToClipboard))
	);

	// Only show "Copy Highlighted Path" if there are matched nodes
	if (MatchedNodes.Num() > 0)
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("CopyHighlightedPath", "Copy Highlighted Path"),
			LOCTEXT("CopyHighlightedPathTooltip", "Copy only the highlighted (blue) path to clipboard"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SDependencyGraphPanel::CopyHighlightedPathToClipboard))
		);
	}

	return MenuBuilder.MakeWidget();
}

void SDependencyGraphPanel::CopyTreeToClipboardHelper(TSharedPtr<FGraphNode> Node, bool bOnlyHighlighted, bool bShowSelectedMarker)
{
	if (!Node.IsValid())
	{
		return;
	}

	FString TreeText;

	// Build referencers (left side) - start at max depth and work outward to leaves at indent 0
	FString ReferencerText;
	TSet<TSharedPtr<FGraphNode>> ReferencerVisited;
	BuildReferencerTreeTextRecursive(Node, CurrentReferencerDepth, ReferencerVisited, ReferencerText, bOnlyHighlighted, 0, CurrentReferencerDepth);

	// Build dependencies (right side)
	FString DependencyText;
	TSet<TSharedPtr<FGraphNode>> DependencyVisited;
	BuildTreeTextRecursive(Node, 0, DependencyVisited, DependencyText, bOnlyHighlighted, 0, CurrentDependencyDepth + 1);

	// Combine sections
	if (!ReferencerText.IsEmpty())
	{
		TreeText += bOnlyHighlighted ? TEXT("=== REFERENCERS (paths to selected) ===\n") : TEXT("=== REFERENCERS ===\n");
		TreeText += ReferencerText;
		TreeText += TEXT("\n");
	}

	// Add explicit [SELECTED] marker if requested
	if (bShowSelectedMarker)
	{
		FString SelectedNodeInfo;
		if (Node->bIsContainer)
		{
			SelectedNodeInfo = FString::Printf(TEXT("[SELECTED] %s (Container)\n"), *Node->ContainerName);
		}
		else if (Node->AssetInfo.IsValid())
		{
			FString ChunkIdStr;
			Node->AssetInfo->ChunkId.ToString(ChunkIdStr);
			SelectedNodeInfo = FString::Printf(TEXT("[SELECTED] %s (0x%s) %s\n"),
				*Node->AssetInfo->FileName,
				*ChunkIdStr,
				*Node->AssetInfo->ContainerName);
		}
		TreeText += SelectedNodeInfo;
	}

	if (!DependencyText.IsEmpty())
	{
		if (bShowSelectedMarker)
		{
			TreeText += TEXT("\n");
		}
		TreeText += bOnlyHighlighted ? TEXT("=== DEPENDENCIES (paths from selected) ===\n") : TEXT("=== DEPENDENCIES ===\n");
		TreeText += DependencyText;
	}

	if (!TreeText.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*TreeText);
	}
}

void SDependencyGraphPanel::CopyEntireGraphToClipboard()
{
	if (!RootNode.IsValid())
	{
		// Fallback: For "All Assets" view or "Container" view, list all nodes
		if (AllNodes.Num() > 0)
		{
			FString TreeText;
			TSet<TSharedPtr<FGraphNode>> Visited;
			for (const TSharedPtr<FGraphNode>& Node : AllNodes)
			{
				if (Node.IsValid() && !Visited.Contains(Node))
				{
					BuildTreeTextRecursive(Node, 0, Visited, TreeText, false, 0, MaxClipboardDepth);
				}
			}
			if (!TreeText.IsEmpty())
			{
				FPlatformApplicationMisc::ClipboardCopy(*TreeText);
			}
		}
		return;
	}

	CopyTreeToClipboardHelper(RootNode, /*bOnlyHighlighted=*/false, /*bShowSelectedMarker=*/true);
}

void SDependencyGraphPanel::CopyHighlightedPathToClipboard()
{
	if (!SelectedNode.IsValid() || MatchedNodes.Num() == 0)
	{
		return;
	}

	CopyTreeToClipboardHelper(SelectedNode, /*bOnlyHighlighted=*/true, /*bShowSelectedMarker=*/true);
}

void SDependencyGraphPanel::CopyNodeAssetToClipboard()
{
	if (!RightClickedNode.IsValid())
	{
		return;
	}

	FString AssetText;

	// Handle container nodes
	if (RightClickedNode->bIsContainer)
	{
		// Count total dependencies from this container (using parallel arrays)
		int32 TotalHardDeps = 0;
		int32 TotalSoftDeps = 0;
		for (int32 DepIndex = 0; DepIndex < RightClickedNode->HardDependencies.Num(); ++DepIndex)
		{
			if (RightClickedNode->HardDependencies[DepIndex].Pin().IsValid())
			{
				if (DepIndex < RightClickedNode->HardDependencyCounts.Num())
				{
					TotalHardDeps += RightClickedNode->HardDependencyCounts[DepIndex];
				}
			}
		}
		for (int32 DepIndex = 0; DepIndex < RightClickedNode->SoftDependencies.Num(); ++DepIndex)
		{
			if (RightClickedNode->SoftDependencies[DepIndex].Pin().IsValid())
			{
				if (DepIndex < RightClickedNode->SoftDependencyCounts.Num())
				{
					TotalSoftDeps += RightClickedNode->SoftDependencyCounts[DepIndex];
				}
			}
		}

		AssetText = FString::Printf(
			TEXT("Container: %s\n")
			TEXT("Total Hard Dependencies: %d\n")
			TEXT("Total Soft Dependencies: %d"),
			*RightClickedNode->ContainerName,
			TotalHardDeps,
			TotalSoftDeps
		);
	}
	// Handle regular asset nodes
	else if (RightClickedNode->AssetInfo.IsValid())
	{
		const TSharedPtr<FIoStoreAssetInfo>& Asset = RightClickedNode->AssetInfo;

		// Convert ChunkId to string
		FString ChunkIdStr;
		Asset->ChunkId.ToString(ChunkIdStr);

		// Convert PackageId to string
		FString PackageIdStr = Asset->PackageId.IsValid() ?
			FString::Printf(TEXT("%016llX"), Asset->PackageId.Value()) :
			TEXT("Invalid");

		AssetText = FString::Printf(
			TEXT("Asset: %s\n")
			TEXT("Chunk ID: %s\n")
			TEXT("Package ID: %s\n")
			TEXT("Container: %s\n")
			TEXT("Size: %.2f MB\n")
			TEXT("Compressed: %.2f MB\n")
			TEXT("Compression: %s\n")
			TEXT("Partition: %d\n")
			TEXT("Hard Dependencies: %d\n")
			TEXT("Soft Dependencies: %d"),
			*Asset->FileName,
			*ChunkIdStr,
			*PackageIdStr,
			*Asset->ContainerName,
			Asset->Size / (1024.0f * 1024.0f),
			Asset->CompressedSize / (1024.0f * 1024.0f),
			Asset->bIsCompressed ? TEXT("Yes") : TEXT("No"),
			Asset->PartitionIndex,
			Asset->HardDependencies.Num(),
			Asset->SoftDependencies.Num()
		);

		// Add hard dependencies list
		if (Asset->HardDependencies.Num() > 0)
		{
			AssetText += TEXT("\n\nHard Dependencies:\n");
			for (const FAssetDependency& HardDep : Asset->HardDependencies)
			{
				// Try to find the dependency asset to get its chunk ID
				TSharedPtr<FIoStoreAssetInfo>* DepAssetPtr = PackageIdToAsset.Find(HardDep.PackageId);
				if (DepAssetPtr && DepAssetPtr->IsValid())
				{
					FString DepChunkIdStr;
					(*DepAssetPtr)->ChunkId.ToString(DepChunkIdStr);
					AssetText += FString::Printf(TEXT("  - %s (0x%s)\n"),
						*(*DepAssetPtr)->FileName,
						*DepChunkIdStr);
				}
				else
				{
					// If we can't find the asset, just show the package name
					AssetText += FString::Printf(TEXT("  - %s\n"), *HardDep.PackageName);
				}
			}
		}

		// Add soft dependencies list
		if (Asset->SoftDependencies.Num() > 0)
		{
			AssetText += TEXT("\nSoft Dependencies:\n");
			for (const FAssetDependency& SoftDep : Asset->SoftDependencies)
			{
				// Try to find the dependency asset to get its chunk ID
				TSharedPtr<FIoStoreAssetInfo>* DepAssetPtr = PackageIdToAsset.Find(SoftDep.PackageId);
				if (DepAssetPtr && DepAssetPtr->IsValid())
				{
					FString DepChunkIdStr;
					(*DepAssetPtr)->ChunkId.ToString(DepChunkIdStr);
					AssetText += FString::Printf(TEXT("  - %s (0x%s)\n"),
						*(*DepAssetPtr)->FileName,
						*DepChunkIdStr);
				}
				else
				{
					// If we can't find the asset, just show the package name
					AssetText += FString::Printf(TEXT("  - %s\n"), *SoftDep.PackageName);
				}
			}
		}
	}

	if (!AssetText.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*AssetText);
	}
}

void SDependencyGraphPanel::BuildTreeTextRecursive(TSharedPtr<FGraphNode> Node, int32 Indent, TSet<TSharedPtr<FGraphNode>>& Visited, FString& OutText, bool bOnlyHighlighted, int32 CurrentDepth, int32 MaxDepth) const
{
	if (!Node.IsValid() || Visited.Contains(Node))
	{
		return;
	}

	// Check depth limit
	if (CurrentDepth >= MaxDepth)
	{
		return;
	}

	Visited.Add(Node);

	// Create indentation
	FString IndentStr;
	for (int32 i = 0; i < Indent; ++i)
	{
		IndentStr += TEXT("   ");
	}

	// Format node information
	FString NodeInfo;
	if (Node->bIsContainer)
	{
		NodeInfo = FString::Printf(TEXT("%s%s (Container)\n"), *IndentStr, *Node->ContainerName);
	}
	else if (Node->AssetInfo.IsValid())
	{
		FString ChunkIdStr;
		Node->AssetInfo->ChunkId.ToString(ChunkIdStr);

		NodeInfo = FString::Printf(TEXT("%s%s (0x%s) %s\n"),
			*IndentStr,
			*Node->AssetInfo->FileName,
			*ChunkIdStr,
			*Node->AssetInfo->ContainerName);
	}

	OutText += NodeInfo;

	// Process dependencies
	TArray<TSharedPtr<FGraphNode>> DepsToProcess;

	// Combine hard and soft dependencies
	for (const TWeakPtr<FGraphNode>& WeakDep : Node->HardDependencies)
	{
		if (TSharedPtr<FGraphNode> Dep = WeakDep.Pin())
		{
			DepsToProcess.Add(Dep);
		}
	}
	for (const TWeakPtr<FGraphNode>& WeakDep : Node->SoftDependencies)
	{
		if (TSharedPtr<FGraphNode> Dep = WeakDep.Pin())
		{
			DepsToProcess.Add(Dep);
		}
	}

	for (const TSharedPtr<FGraphNode>& Dep : DepsToProcess)
	{
		if (!Dep.IsValid())
		{
			continue;
		}

		// If only showing highlighted path, check if this edge is highlighted
		if (bOnlyHighlighted)
		{
			bool bShouldInclude = false;

			if (SelectedNode.IsValid())
			{
				bool bIsRightSide = Node->Position.X >= SelectedNode->Position.X;

				if (bIsRightSide)
				{
					// Right side: use dependency path check
					bShouldInclude = NodeReachableFromSelected.FindRef(Node) && NodeCanReachMatched.FindRef(Dep);
				}
				else
				{
					// Left side: use referencer path check
					bShouldInclude = NodeReachableFromMatched.FindRef(Node) && NodeReachableFromMatched.FindRef(Dep);
				}
			}

			// Skip this dependency if it's not on the highlighted path
			if (!bShouldInclude)
			{
				continue;
			}
		}

		BuildTreeTextRecursive(Dep, Indent + 1, Visited, OutText, bOnlyHighlighted, CurrentDepth + 1, MaxDepth);
	}
}

void SDependencyGraphPanel::BuildReferencerTreeTextRecursive(TSharedPtr<FGraphNode> Node, int32 Indent, TSet<TSharedPtr<FGraphNode>>& Visited, FString& OutText, bool bOnlyHighlighted, int32 CurrentDepth, int32 MaxDepth) const
{
	if (!Node.IsValid())
	{
		return;
	}

	// Check depth limit
	if (CurrentDepth >= MaxDepth)
	{
		return;
	}

	// Collect all unique reference paths from leaf nodes to the selected node
	TArray<TArray<TSharedPtr<FGraphNode>>> AllPaths;
	TArray<TSharedPtr<FGraphNode>> CurrentPath;
	TSet<TSharedPtr<FGraphNode>> PathVisited;

	CollectReferencePaths(Node, CurrentPath, PathVisited, AllPaths, bOnlyHighlighted, 0, MaxDepth);

	// Print each path separately
	int32 PathIndex = 1;
	for (const TArray<TSharedPtr<FGraphNode>>& Path : AllPaths)
	{
		if (Path.Num() == 0)
		{
			continue;
		}

		// Header for each path
		if (AllPaths.Num() > 1)
		{
			OutText += FString::Printf(TEXT("\nReference Path %d of %d:\n"), PathIndex, AllPaths.Num());
		}

		// Print each node in the path in reverse order (leaf -> parent -> selected)
		// This shows the chain of references leading TO the selected asset
		for (int32 i = Path.Num() - 1; i >= 0; --i)
		{
			const TSharedPtr<FGraphNode>& PathNode = Path[i];

			// Simple indentation (3 spaces per level, matching dependency tree format)
			// Deepest indent for selected (i=0), least indent for leaf (i=Path.Num()-1)
			int32 IndentLevel = Path.Num() - 1 - i;
			FString IndentStr;
			for (int32 j = 0; j < IndentLevel; ++j)
			{
				IndentStr += TEXT("   ");
			}

			// Format node information (matching dependency tree format)
			if (PathNode->bIsContainer)
			{
				OutText += FString::Printf(TEXT("%s%s (Container)\n"), *IndentStr, *PathNode->ContainerName);
			}
			else if (PathNode->AssetInfo.IsValid())
			{
				FString ChunkIdStr;
				PathNode->AssetInfo->ChunkId.ToString(ChunkIdStr);
				OutText += FString::Printf(TEXT("%s%s (0x%s) %s\n"),
					*IndentStr,
					*PathNode->AssetInfo->FileName,
					*ChunkIdStr,
					*PathNode->AssetInfo->ContainerName);
			}
		}

		PathIndex++;
	}
}

void SDependencyGraphPanel::CollectReferencePaths(
	TSharedPtr<FGraphNode> Node,
	TArray<TSharedPtr<FGraphNode>>& CurrentPath,
	TSet<TSharedPtr<FGraphNode>>& PathVisited,
	TArray<TArray<TSharedPtr<FGraphNode>>>& OutPaths,
	bool bOnlyHighlighted,
	int32 CurrentDepth,
	int32 MaxDepth) const
{
	if (!Node.IsValid() || PathVisited.Contains(Node))
	{
		return;
	}

	// Check depth limit (CurrentDepth is number of referencer levels from selected node)
	if (CurrentDepth > MaxDepth)
	{
		return;
	}

	// Add current node to the path
	CurrentPath.Add(Node);
	PathVisited.Add(Node);

	// Find all nodes that reference this node using ReverseEdges map (O(E) instead of O(N²))
	TArray<TSharedPtr<FGraphNode>> Referencers;
	if (const TArray<TSharedPtr<FGraphNode>>* IncomingNodes = ReverseEdges.Find(Node))
	{
		for (const TSharedPtr<FGraphNode>& OtherNode : *IncomingNodes)
		{
			if (!OtherNode.IsValid())
			{
				continue;
			}

			// If only showing highlighted path, check if this edge is highlighted
			if (bOnlyHighlighted)
			{
				bool bShouldInclude = NodeReachableFromMatched.FindRef(OtherNode) && NodeReachableFromMatched.FindRef(Node);
				if (!bShouldInclude)
				{
					continue;
				}
			}

			Referencers.Add(OtherNode);
		}
	}

	// If this is a leaf node (no referencers), save the current path
	if (Referencers.Num() == 0)
	{
		OutPaths.Add(CurrentPath);
	}
	else
	{
		// Recursively explore each referencer at next depth level
		for (const TSharedPtr<FGraphNode>& Referencer : Referencers)
		{
			CollectReferencePaths(Referencer, CurrentPath, PathVisited, OutPaths, bOnlyHighlighted, CurrentDepth + 1, MaxDepth);
		}
	}

	// Backtrack
	CurrentPath.Pop();
	PathVisited.Remove(Node);
}

#undef LOCTEXT_NAMESPACE
