// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SCanvas.h"
#include "SIoStoreDependencyViewer.h"

/** Represents a node in the visual graph */
struct FGraphNode
{
	TSharedPtr<FIoStoreAssetInfo> AssetInfo;
	FVector2D Position;
	FVector2D Size;
	TArray<TWeakPtr<FGraphNode>> HardDependencies;
	TArray<TWeakPtr<FGraphNode>> SoftDependencies;
	bool bIsRoot = false;

	// For container view
	FString ContainerName;
	bool bIsContainer = false;
	int32 HardDependencyCount = 0;  // Deprecated: Use HardDependencyCounts array for accurate per-edge counts
	int32 SoftDependencyCount = 0;  // Deprecated: Use SoftDependencyCounts array for accurate per-edge counts

	// Edge-specific dependency counts (for container view)
	// Parallel arrays to HardDependencies/SoftDependencies: count at index i corresponds to dependency at index i
	// Prevents overwriting when multiple sources point to same target
	// Using parallel arrays instead of TMap to avoid TSharedPtr reference cycles
	TArray<int32> HardDependencyCounts;
	TArray<int32> SoftDependencyCounts;

	FGraphNode() : Position(0, 0), Size(200, 80) {}
};

/** Custom widget for displaying dependency graph */
class SDependencyGraphPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SDependencyGraphPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SDependencyGraphPanel();

	/** Rebuild the graph from a root asset */
	void RebuildGraph(TSharedPtr<FIoStoreAssetInfo> RootAsset,
		const TMap<FPackageId, TSharedPtr<FIoStoreAssetInfo>>& InPackageIdToAsset,
		int32 DepthLimit,
		int32 ReferencerDepthLimit = 0);

	/** Rebuild graph showing all assets */
	void RebuildGraphAllAssets(const TArray<TSharedPtr<FIoStoreAssetInfo>>& Assets,
		TSharedPtr<FIoStoreAssetInfo> SelectedAsset);

	/** Rebuild graph showing container dependencies */
	void RebuildGraphContainers(const TArray<TSharedPtr<FIoStoreAssetInfo>>& AllAssets);

	/** Zoom to fit all nodes */
	void ZoomToFit();

	/** Paint the graph */
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect,
		FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Context menu for graph panel */
	TSharedPtr<SWidget> OnGraphContextMenuOpening();
	void CopyEntireGraphToClipboard();
	void CopyHighlightedPathToClipboard();
	void CopyNodeAssetToClipboard();

	/** Track right-clicked node for context menu */
	TSharedPtr<FGraphNode> RightClickedNode;

	/** Set the selected node */
	void SetSelectedNode(TSharedPtr<FGraphNode> Node) { SelectedNode = Node; }

	/** Get the currently selected node */
	TSharedPtr<FGraphNode> GetSelectedNode() const { return SelectedNode; }

	/** Search functionality */
	void OnGraphSearchTextChanged(const FText& InText);
	FText GetGraphSearchText() const { return GraphSearchText; }

	/** Get tooltip text for hovered node */
	FText GetTooltipText() const;

	/** Set the referencer map for building referencer nodes */
	void SetReferencerMap(const TMap<FPackageId, TArray<FAssetDependency>>& InReferencerMap) { PackageIdToReferencers = InReferencerMap; }

private:
	/** Recursively create nodes for dependencies */
	void CreateDependencyNodes(TSharedPtr<FGraphNode> ParentNode, TSharedPtr<FIoStoreAssetInfo> ParentAsset,
		int32 CurrentDepth, int32 MaxDepth, TMap<FPackageId, TSharedPtr<FGraphNode>>& CreatedNodes,
		float& CurrentX, float& CurrentY, bool bIsRootCall = false);

	/** Recursively create nodes for referencers */
	void CreateReferencerNodes(TSharedPtr<FGraphNode> ChildNode, TSharedPtr<FIoStoreAssetInfo> ChildAsset,
		int32 CurrentDepth, int32 MaxDepth, TMap<FPackageId, TSharedPtr<FGraphNode>>& CreatedNodes,
		const TMap<FPackageId, TArray<FAssetDependency>>& ReferencerMap,
		float& CurrentX, float& CurrentY, bool bIsRootCall = false);

	/** Draw a node */
	void DrawNode(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32& LayerId,
		TSharedPtr<FGraphNode> Node) const;

	/** Draw a connection between nodes */
	void DrawConnection(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32& LayerId,
		const FVector2D& Start, const FVector2D& End, const FLinearColor& Color, float Thickness) const;

	/** Get screen space position for a node */
	FVector2D GetNodeScreenPosition(TSharedPtr<FGraphNode> Node) const;

	/** All nodes in the graph */
	TArray<TSharedPtr<FGraphNode>> AllNodes;

	/** Root node */
	TSharedPtr<FGraphNode> RootNode;

	/** Package ID to asset lookup */
	TMap<FPackageId, TSharedPtr<FIoStoreAssetInfo>> PackageIdToAsset;

	/** Package ID to referencers lookup */
	TMap<FPackageId, TArray<FAssetDependency>> PackageIdToReferencers;

	/** View transform */
	FVector2D ViewOffset;
	float ZoomLevel;

	/** Interaction state */
	bool bIsPanning;
	FVector2D LastMousePosition;
	FVector2D LastClickPosition;
	double LastClickTime;

	/** Selected and hovered nodes */
	TSharedPtr<FGraphNode> SelectedNode;
	TSharedPtr<FGraphNode> HoveredNode;

	/** Find node at screen position */
	TSharedPtr<FGraphNode> FindNodeAtPosition(const FVector2D& ScreenPosition, const FGeometry& MyGeometry) const;

	/** Perform search (private helper) */
	void PerformSearch();

	/** Helper to copy tree to clipboard with specified options */
	void CopyTreeToClipboardHelper(TSharedPtr<FGraphNode> Node, bool bOnlyHighlighted, bool bShowSelectedMarker);

	/** Build tree text representation */
	void BuildTreeTextRecursive(TSharedPtr<FGraphNode> Node, int32 Indent, TSet<TSharedPtr<FGraphNode>>& Visited, FString& OutText, bool bOnlyHighlighted, int32 CurrentDepth = 0, int32 MaxDepth = 999) const;
	void BuildReferencerTreeTextRecursive(TSharedPtr<FGraphNode> Node, int32 Indent, TSet<TSharedPtr<FGraphNode>>& Visited, FString& OutText, bool bOnlyHighlighted, int32 CurrentDepth = 0, int32 MaxDepth = 999) const;

	/** Helper to collect all unique reference paths from leaf nodes to target node */
	void CollectReferencePaths(TSharedPtr<FGraphNode> Node, TArray<TSharedPtr<FGraphNode>>& CurrentPath, TSet<TSharedPtr<FGraphNode>>& PathVisited, TArray<TArray<TSharedPtr<FGraphNode>>>& OutPaths, bool bOnlyHighlighted, int32 CurrentDepth, int32 MaxDepth) const;

	/** Build reverse edge map for efficient incoming-edge queries (const to support mutable cache pattern) */
	void BuildReverseEdges() const;

	/** Compute paths from selected to matched nodes */
	void ComputePathsToMatchedNodes();
	bool IsReachableFromSelected(TSharedPtr<FGraphNode> Node, TSet<TSharedPtr<FGraphNode>>& Visited) const;
	bool CanReachMatchedNode(TSharedPtr<FGraphNode> Node, TSet<TSharedPtr<FGraphNode>>& Visited) const;
	bool CanReachNode(TSharedPtr<FGraphNode> FromNode, TSharedPtr<FGraphNode> ToNode, TSet<TSharedPtr<FGraphNode>>& Visited) const;
	bool IsReachableFromMatched(TSharedPtr<FGraphNode> Node, TSet<TSharedPtr<FGraphNode>>& Visited) const;
	bool CanReachSelected(TSharedPtr<FGraphNode> Node, TSet<TSharedPtr<FGraphNode>>& Visited) const;

	/** Search state */
	FText GraphSearchText;
	TSet<TSharedPtr<FGraphNode>> MatchedNodes;

	/** Path highlighting state - nodes that can reach matched nodes */
	mutable TMap<TSharedPtr<FGraphNode>, bool> NodeCanReachMatched;
	mutable TMap<TSharedPtr<FGraphNode>, bool> NodeReachableFromSelected;
	mutable TMap<TSharedPtr<FGraphNode>, bool> NodeReachableFromMatched;
	mutable TMap<TSharedPtr<FGraphNode>, bool> NodeCanReachSelected;

	/** Reverse edge map: node -> set of nodes that depend on it (for efficient incoming edge queries) */
	mutable TMap<TSharedPtr<FGraphNode>, TArray<TSharedPtr<FGraphNode>>> ReverseEdges;

	/** Layout constants */
	static constexpr float NodeWidth = 200.0f;
	static constexpr float NodeHeight = 80.0f;
	static constexpr float HorizontalSpacing = 250.0f;
	static constexpr float VerticalSpacing = 120.0f;
	static constexpr int32 MaxClipboardDepth = 50;  // Maximum depth for clipboard copy operations

	/** Current depth limits for copy functions */
	int32 CurrentDependencyDepth = 1;
	int32 CurrentReferencerDepth = 0;
};
