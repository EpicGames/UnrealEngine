// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/SListView.h"
#include "IO/IoContainerId.h"
#include "IO/IoChunkId.h"
#include "IO/IoDispatcher.h"
#include "IO/IoContainerHeader.h"
#include "IO/OnDemandToc.h"
#include "Misc/AES.h"

class ITableRow;
class STableViewBase;
class STextBlock;
class SSearchBox;
class SDependencyGraphPanel;
class FIoContainerMetaReader;
template <typename OptionType> class SComboBox;

/** Represents a dependency relationship */
struct FAssetDependency
{
	FPackageId PackageId;
	FString PackageName;
	bool bIsSoftReference = false; // false = hard, true = soft

	FAssetDependency() = default;
	FAssetDependency(FPackageId InId, bool bSoft) : PackageId(InId), bIsSoftReference(bSoft) {}
};

/** Represents a chunk/asset in the container */
struct FIoStoreAssetInfo
{
	FIoChunkId ChunkId;
	FPackageId PackageId;
	FString FileName;
	FString PackageName;
	FString ContainerName;
	uint64 Size = 0;
	uint64 CompressedSize = 0;
	bool bIsCompressed = false;
	TArray<FAssetDependency> HardDependencies;
	TArray<FAssetDependency> SoftDependencies;
	TArray<FString> Tags;
	int32 PartitionIndex = 0;

	FIoStoreAssetInfo() = default;
};


/**
 * IoStore Dependency Viewer Widget
 * Loads and displays IoStore container files (.utoc, .uondemandtoc)
 */
class SIoStoreDependencyViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SIoStoreDependencyViewer) {}
		SLATE_ARGUMENT(FString, InitialPath)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SIoStoreDependencyViewer();

private:
	// Menu actions
	FReply OnLoadDirectoryClicked();
	FReply OnRefreshClicked();
	FReply OnClearClicked();
	FReply OnCloudDownloadClicked();
	FReply OnLoadCloudBuildClicked();

	// Loading functions
	void ClearAllData(); // Clears all data and UI state
	bool LoadFromDirectory(const FString& DirectoryPath);
	bool LoadAllContainersWithProgress(const FString& DirectoryPath);
	bool LoadOnDemandContainers(const FString& DirectoryPath, const TMap<FGuid, FAES::FAESKey>& EncryptionKeys);
	bool LoadMetaData(const FString& DirectoryPath);
	void ProcessContainerHeader(const FIoContainerHeader& Header, const FString& ContainerName);
	void ResolveDependencyNames();

	// Asset list callbacks
	TSharedRef<ITableRow> OnGenerateRowForAssetList(TSharedPtr<FIoStoreAssetInfo> AssetInfo, const TSharedRef<STableViewBase>& OwnerTable);
	void OnAssetListSelectionChanged(TSharedPtr<FIoStoreAssetInfo> AssetInfo, ESelectInfo::Type SelectInfo);

	// Dependency list callbacks
	TSharedRef<ITableRow> OnGenerateRowForDependencyList(TSharedPtr<FAssetDependency> Dependency, const TSharedRef<STableViewBase>& OwnerTable);
	void OnDependencyListSelectionChanged(TSharedPtr<FAssetDependency> Dependency, ESelectInfo::Type SelectInfo);

	// Search
	void OnSearchTextChanged(const FText& InText);
	void OnSearchModeChanged(int32 NewMode);

	// Graph controls
	void OnDepthChanged(int32 NewDepth);
	int32 GetCurrentDepth() const { return CurrentTreeDepth; }
	void OnReferencerDepthChanged(int32 NewDepth);
	int32 GetCurrentReferencerDepth() const { return CurrentReferencerDepth; }
	FReply OnZoomToFitClicked();
	void OnGraphViewModeChanged(TSharedPtr<FString> NewMode, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> OnGenerateGraphViewModeWidget(TSharedPtr<FString> InItem);
	FText GetCurrentGraphViewModeText() const;
	void OnGraphSearchTextChanged(const FText& InText);

	// UI update functions
	void UpdateAssetList();
	void UpdateAssetInfoPanel(TSharedPtr<FIoStoreAssetInfo> AssetInfo);
	void BuildDependencyGraph(TSharedPtr<FIoStoreAssetInfo> RootAsset);

	// Navigation
	FReply OnNavigateBackClicked();
	FReply OnNavigateForwardClicked();
	bool CanNavigateBack() const;
	bool CanNavigateForward() const;
	void NavigateToAsset(TSharedPtr<FIoStoreAssetInfo> Asset, bool bAddToHistory);

	// Context menu handlers
	TSharedPtr<SWidget> OnReferencersContextMenuOpening();
	TSharedPtr<SWidget> OnHardDepsContextMenuOpening();
	TSharedPtr<SWidget> OnSoftDepsContextMenuOpening();
	void CopyReferencersToClipboard();
	void CopyHardDepsToClipboard();
	void CopySoftDepsToClipboard();

	// Helper functions
	TSharedPtr<FIoStoreAssetInfo> FindAssetByPackageId(const FPackageId& PackageId);
	TSharedPtr<FIoStoreAssetInfo> FindAssetByChunkId(const FIoChunkId& ChunkId);
	FString FindDirectoryWithTocFiles(const FString& RootDirectory);

	// UI state
	TSharedPtr<SListView<TSharedPtr<FIoStoreAssetInfo>>> AssetListView;
	TSharedPtr<SListView<TSharedPtr<FAssetDependency>>> HardDependenciesListView;
	TSharedPtr<SListView<TSharedPtr<FAssetDependency>>> SoftDependenciesListView;
	TSharedPtr<SListView<TSharedPtr<FAssetDependency>>> ReferencersListView;
	TSharedPtr<STextBlock> StatusText;
	TSharedPtr<STextBlock> AssetInfoText;
	TSharedPtr<SDependencyGraphPanel> GraphPanel;
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> GraphViewModeComboBox;
	TArray<TSharedPtr<FString>> GraphViewModeOptions;

	// Data
	TArray<TSharedPtr<FIoStoreAssetInfo>> AllAssets;
	TArray<TSharedPtr<FIoStoreAssetInfo>> FilteredAssets;
	TArray<TSharedPtr<FAssetDependency>> CurrentHardDependencies;
	TArray<TSharedPtr<FAssetDependency>> CurrentSoftDependencies;
	TArray<TSharedPtr<FAssetDependency>> CurrentReferencers;
	TMap<FIoChunkId, TSharedPtr<FIoStoreAssetInfo>> ChunkIdToAsset;
	TMap<FPackageId, TSharedPtr<FIoStoreAssetInfo>> PackageIdToAsset;
	TMap<FPackageId, TArray<FAssetDependency>> PackageIdToReferencers; // Reverse dependency map
	TSharedPtr<FIoStoreAssetInfo> CurrentSelectedAsset;

	// Navigation history for back/forward
	TArray<TSharedPtr<FIoStoreAssetInfo>> NavigationHistory;
	int32 NavigationHistoryIndex = -1;
	bool bIsNavigating = false;  // Prevent adding to history during navigation

	// Shared data lock for parallel container loading
	FCriticalSection SharedDataLock;  // Protects AllAssets, ChunkIdToAsset, PackageIdToAsset, PackageIdToReferencers

	// Search state
	FString CurrentSearchText;
	int32 SearchMode = 0; // 0=Name, 1=ChunkId, 2=Tags

	// Graph view mode
	enum class EGraphViewMode
	{
		SelectedAssetDependencies,
		AllAssets,
		ContainerDependencies
	};
	EGraphViewMode CurrentGraphViewMode = EGraphViewMode::SelectedAssetDependencies;

	// Tree depth
	int32 CurrentTreeDepth = 1;
	int32 CurrentReferencerDepth = 0;

	// IoStore data
	TArray<TUniquePtr<FIoStoreReader>> IoStoreReaders; // Legacy - keeping for now
	TArray<TSharedPtr<class FIoStoreReaderAdapter>> IoStoreAdapters; // New adapter-based readers
	TSharedPtr<FIoContainerMetaReader> MetaReader;

	// Cloud build parameters (for partial downloads)
	bool bUsePartialDownloads = false;
	FString ZenExePath;
	FString OidcExePath;
	FString CloudNamespace;
	FString CloudBucket;
	FString CloudBuildId;
	FString CloudProxyUrl;
	FString BaseDownloadDirectory; // Where UTOCs are stored

	// Current loaded build info (for status display)
	FString LoadedBuildSource; // Path or cloud build info

	static constexpr int32 MaxTreeDepth = 10;

public:
	/**
	 * Configure cloud build parameters for partial downloads
	 * Call this before LoadFromDirectory if using partial downloads
	 */
	void SetCloudBuildParameters(
		const FString& InZenExePath,
		const FString& InOidcExePath,
		const FString& InNamespace,
		const FString& InBucket,
		const FString& InBuildId,
		const FString& InProxyUrl,
		const FString& InBaseDownloadDirectory,
		bool bInUsePartialDownloads)
	{
		ZenExePath = InZenExePath;
		OidcExePath = InOidcExePath;
		CloudNamespace = InNamespace;
		CloudBucket = InBucket;
		CloudBuildId = InBuildId;
		CloudProxyUrl = InProxyUrl;
		BaseDownloadDirectory = InBaseDownloadDirectory;
		bUsePartialDownloads = bInUsePartialDownloads;
	}
};
