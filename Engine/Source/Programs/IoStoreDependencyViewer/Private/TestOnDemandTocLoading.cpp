// Copyright Epic Games, Inc. All Rights Reserved.

#include "IoStoreDependencyViewer.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"

void TestOnDemandTocLoading(const FString& DirectoryPath)
{
	UE_LOGF(LogIoStoreDependencyViewer, Display, "=== Testing OnDemand TOC Loading ===");
	UE_LOGF(LogIoStoreDependencyViewer, Display, "Directory Path: %ls", *DirectoryPath);

	// Verify directory exists
	if (!FPaths::DirectoryExists(DirectoryPath))
	{
		UE_LOGF(LogIoStoreDependencyViewer, Error, "Directory does not exist: %ls", *DirectoryPath);
		return;
	}

	// Search for .utoc files
	TArray<FString> UtocFiles;
	IFileManager::Get().FindFiles(UtocFiles, *FPaths::Combine(DirectoryPath, TEXT("*.utoc")), true, false);

	UE_LOGF(LogIoStoreDependencyViewer, Display, "Found %d .utoc files", UtocFiles.Num());

	for (const FString& UtocFile : UtocFiles)
	{
		FString FullPath = FPaths::Combine(DirectoryPath, UtocFile);
		UE_LOGF(LogIoStoreDependencyViewer, Display, "  %ls", *UtocFile);
	}

	// Search for .uondemandtoc files
	TArray<FString> OnDemandTocFiles;
	IFileManager::Get().FindFiles(OnDemandTocFiles, *FPaths::Combine(DirectoryPath, TEXT("*.uondemandtoc")), true, false);

	UE_LOGF(LogIoStoreDependencyViewer, Display, "Found %d .uondemandtoc files", OnDemandTocFiles.Num());

	for (const FString& OnDemandTocFile : OnDemandTocFiles)
	{
		FString FullPath = FPaths::Combine(DirectoryPath, OnDemandTocFile);
		UE_LOGF(LogIoStoreDependencyViewer, Display, "  %ls", *OnDemandTocFile);
	}

	UE_LOGF(LogIoStoreDependencyViewer, Display, "=== OnDemand TOC Loading Test Complete ===");
}
