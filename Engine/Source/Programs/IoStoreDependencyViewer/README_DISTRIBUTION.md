# IoStoreDependencyViewer

A standalone tool for analyzing and visualizing IoStore container dependencies in Unreal Engine projects.

## Features

- **Dependency Graph Visualization**: View asset dependencies and referencers in an interactive graph
- **Search & Highlighting**: Use regex search to find and highlight specific assets and their paths
- **Container View**: Analyze dependencies between different pak chunks/containers
- **Copy Functions**: Export dependency trees to clipboard for documentation
- **Configurable Depth**: Control how many levels of dependencies/referencers to display

## Usage

1. Extract all files from the zip to a folder
2. Run `IoStoreDependencyViewer.exe`
3. Click "Load IoStore" and navigate to your cooked build's Paks folder
4. Select the `.umeta` file (e.g., `global.umeta`)

## UI Controls

### Main Controls
- **Dependency Depth**: Set how many levels of dependencies to show (right side of graph)
- **Referencer Depth**: Set how many levels of referencers to show (left side of graph)
- **Search Box**: Enter regex patterns to search for assets (e.g., "M_.*" finds all materials starting with M_)

### Graph Interaction
- **Left Click + Drag**: Pan the view
- **Mouse Wheel**: Zoom in/out
- **Double Click**: Center on clicked node
- **Right Click**: Context menu with copy options

### Context Menu Options
- **Copy Entire Tree**: Copy the full dependency tree respecting depth settings
- **Copy Highlighted Path**: Copy only the highlighted (blue) paths from search results
- **Copy Node Asset**: Copy the selected node's asset information

### View Modes
- **Asset View**: Click an asset in the list to see its dependency tree
- **All Assets View**: Show all assets (use search to find specific ones)
- **Container View**: Analyze dependencies between pak chunks

## Search Tips

The search box supports regex patterns:
- `M_.*` - Find all assets starting with "M_" (materials)
- `.*Weapon.*` - Find assets containing "Weapon"
- `T_.*_N$` - Find textures ending with "_N" (normal maps)

When searching:
- Matching nodes turn **yellow**
- Paths from selected to matched nodes turn **blue**
- Use "Copy Highlighted Path" to export only the blue paths

## Troubleshooting

**Tool won't launch**:
- Ensure all DLL files are in the same folder as the .exe
- Run as Administrator if you get permission errors
- Check Windows Event Viewer for specific error messages

**Can't load IoStore**:
- Make sure you're selecting the `.umeta` file (not .utoc or .ucas)
- Verify the cooked build is using IoStore (not legacy pak files)
- Check that all .utoc and .ucas files are present in the same folder

**Graph is too large/slow**:
- Reduce the dependency and referencer depth settings
- Use search to focus on specific assets
- Try Container View for a high-level overview

## Technical Details

**Supported Formats**:
- IoStore TOC version 8 (Unreal Engine 5.x)
- Supports hard and soft package dependencies
- Handles optional packages and external references

**Performance**:
- Can load containers with thousands of assets
- Search uses compiled regex for fast matching
- Graph caching for smooth interaction

## Building from Source

If you need to rebuild the tool:

```batch
cd Engine\Build\BatchFiles
Build.bat IoStoreDependencyViewer Win64 Development
```

The executable will be output to: `Engine\Binaries\Win64\IoStoreDependencyViewer.exe`

## Version History

### v1.0.0
- Initial release
- Dependency and referencer visualization
- Regex search with path highlighting
- Container view
- Copy functions with depth limiting
- Side-based highlighting logic for accurate path display

## License

Copyright Epic Games, Inc. All Rights Reserved.

---

For issues or questions, contact your internal tools team.
