@echo off
REM Test the packaged IoStoreDependencyViewer for missing dependencies

echo.
echo ========================================
echo  IoStoreDependencyViewer Package Test
echo ========================================
echo.

REM Run the PowerShell test script
powershell.exe -ExecutionPolicy Bypass -File "%~dp0TestPackage.ps1"

if %ERRORLEVEL% NEQ 0 (
    echo.
    echo Test failed with error code: %ERRORLEVEL%
    pause
    exit /b %ERRORLEVEL%
)
