# IoStoreDependencyViewer Package Test Script
# This script extracts the package and checks for missing dependencies

$ErrorActionPreference = "Stop"

Write-Host "========================================" -ForegroundColor Cyan
Write-Host " IoStoreDependencyViewer Package Test" -ForegroundColor Cyan
Write-Host "========================================" -ForegroundColor Cyan
Write-Host ""

# Read settings
$SettingsPath = Join-Path $PSScriptRoot "PackageSettings.ini"
if (-not (Test-Path $SettingsPath)) {
    Write-Error "PackageSettings.ini not found"
    exit 1
}

# Parse INI file
$IniContent = Get-Content $SettingsPath
$OutputDir = $null

foreach ($Line in $IniContent) {
    $Line = $Line.Trim()
    if ($Line -match '^OutputPath=(.+)$') {
        $OutputDir = $Matches[1].Trim()
    }
}

if (-not $OutputDir) {
    Write-Error "OutputPath not found in PackageSettings.ini"
    exit 1
}

$PackagePath = Join-Path $OutputDir "IoStoreDependencyViewer.zip"
Write-Host "Package location: $PackagePath" -ForegroundColor White
Write-Host ""

if (-not (Test-Path $PackagePath)) {
    Write-Error "Package not found at: $PackagePath`nPlease run PackageForDistribution.bat first"
    exit 1
}

# Create test directory with timestamp to avoid locks
$Timestamp = Get-Date -Format "yyyyMMdd_HHmmss"
$TestDir = Join-Path $env:TEMP "IoStoreDependencyViewer_Test_$Timestamp"
Write-Host "Creating test directory: $TestDir" -ForegroundColor Cyan

New-Item -ItemType Directory -Path $TestDir | Out-Null

# Extract package
Write-Host "Extracting package..." -ForegroundColor Cyan
try {
    Expand-Archive -Path $PackagePath -DestinationPath $TestDir -Force
    Write-Host "  [OK] Package extracted successfully" -ForegroundColor Green
} catch {
    Write-Error "Failed to extract package: $_"
    exit 1
}

Write-Host ""
Write-Host "Package contents:" -ForegroundColor Cyan
$Files = Get-ChildItem $TestDir
foreach ($File in $Files) {
    $Size = if ($File.PSIsContainer) { "" } else { " ($("{0:N2}" -f ($File.Length / 1MB)) MB)" }
    Write-Host "  $($File.Name)$Size"
}
Write-Host ""

# Display version info
if (Test-Path (Join-Path $TestDir "version.txt")) {
    Write-Host "========================================" -ForegroundColor Yellow
    Get-Content (Join-Path $TestDir "version.txt") | ForEach-Object { Write-Host $_ }
    Write-Host "========================================" -ForegroundColor Yellow
    Write-Host ""
}

# Count files
$ExeCount = ($Files | Where-Object { $_.Extension -eq ".exe" }).Count
$DllCount = ($Files | Where-Object { $_.Extension -eq ".dll" }).Count

Write-Host "Summary:" -ForegroundColor Cyan
Write-Host "  Executables: $ExeCount" -ForegroundColor White
Write-Host "  DLLs: $DllCount" -ForegroundColor White
Write-Host ""

# Check for exe in the Engine/Binaries/Win64 folder
$ExePath = Join-Path $TestDir "Engine\Binaries\Win64\IoStoreDependencyViewer.exe"
if (-not (Test-Path $ExePath)) {
    Write-Error "IoStoreDependencyViewer.exe not found at: $ExePath"
    exit 1
}
$ExeWorkingDir = Split-Path $ExePath -Parent

Write-Host "========================================" -ForegroundColor Cyan
Write-Host "Testing executable launch..." -ForegroundColor Cyan
Write-Host "========================================" -ForegroundColor Cyan
Write-Host ""

# Record time before launch to check for new events
$beforeTime = (Get-Date).AddSeconds(-2)

Write-Host "Attempting to launch IoStoreDependencyViewer.exe..." -ForegroundColor Yellow
Write-Host "(Checking for missing DLL errors...)" -ForegroundColor Yellow
Write-Host ""

try {
    $Process = Start-Process -FilePath $ExePath -WorkingDirectory $ExeWorkingDir -PassThru -ErrorAction Stop

    # Wait for initialization
    Start-Sleep -Seconds 3

    # Check if process is still running
    if (-not $Process.HasExited) {
        Write-Host "Application appears to be running..." -ForegroundColor Green
        Write-Host "Process ID: $($Process.Id)" -ForegroundColor White
        Write-Host ""

        # Kill the process
        Write-Host "Terminating test process..." -ForegroundColor Cyan
        Stop-Process -Id $Process.Id -Force -ErrorAction SilentlyContinue
        Start-Sleep -Seconds 1

        # Check for Application Popup events in System log (missing DLL errors)
        Write-Host "Checking for Application Popup events (missing DLL errors)..." -ForegroundColor Cyan

        try {
            # Query System log for Application Popup events
            $popupEvents = Get-WinEvent -FilterHashtable @{
                LogName = 'System'
                ProviderName = 'Application Popup'
                StartTime = $beforeTime
            } -ErrorAction SilentlyContinue

            if ($popupEvents) {
                Write-Host ""
                Write-Host "====================================" -ForegroundColor Red
                Write-Host "MISSING DLL ERROR DETECTED!" -ForegroundColor Red
                Write-Host "====================================" -ForegroundColor Red
                Write-Host ""

                foreach ($event in $popupEvents) {
                    Write-Host "Time: $($event.TimeCreated)" -ForegroundColor Yellow
                    Write-Host "Message:" -ForegroundColor Yellow
                    Write-Host $event.Message -ForegroundColor White
                    Write-Host ""

                    # Try to extract DLL name from message
                    $message = $event.Message
                    if ($message -match '([A-Za-z0-9_\-]+\.dll)') {
                        $missingDll = $Matches[1]
                        Write-Host "MISSING DLL: $missingDll" -ForegroundColor Red
                        Write-Host ""

                        # Check if it exists in Engine binaries
                        $engineBin = Join-Path (Split-Path -Parent (Split-Path -Parent (Split-Path -Parent $PSScriptRoot))) "Binaries\Win64"
                        $sourcePath = Join-Path $engineBin $missingDll

                        if (Test-Path $sourcePath) {
                            $size = (Get-Item $sourcePath).Length / 1MB
                            Write-Host "FOUND in Engine/Binaries/Win64:" -ForegroundColor Green
                            Write-Host "  Path: $sourcePath" -ForegroundColor White
                            Write-Host "  Size: $("{0:N2}" -f $size) MB" -ForegroundColor White
                            Write-Host ""
                            Write-Host "ACTION REQUIRED:" -ForegroundColor Yellow
                            Write-Host "  1. Add '$missingDll' to the RequiredModules list in PackageForDistribution.ps1" -ForegroundColor Yellow
                            Write-Host "  2. Re-run PackageForDistribution.bat" -ForegroundColor Yellow
                            Write-Host "  3. Re-run this test" -ForegroundColor Yellow
                        } else {
                            Write-Host "NOT FOUND in Engine/Binaries/Win64" -ForegroundColor Red
                            Write-Host "This DLL may need to be obtained from another source." -ForegroundColor Yellow
                        }
                    }
                }

                Write-Host ""
                Write-Host "TEST FAILED: Missing DLL dependency" -ForegroundColor Red
                exit 1
            } else {
                Write-Host "  No Application Popup events found" -ForegroundColor Green
            }
        } catch {
            Write-Host "  Warning: Could not check System event log" -ForegroundColor Yellow
        }

        # Also check Application log for crashes
        Write-Host "Checking Application log for errors..." -ForegroundColor Cyan
        try {
            $appErrors = Get-WinEvent -FilterHashtable @{
                LogName = 'Application'
                Level = 2  # Error
                StartTime = $beforeTime
            } -MaxEvents 10 -ErrorAction SilentlyContinue | Where-Object {
                $_.Message -like "*IoStoreDependencyViewer*" -or
                $_.Message -like "*0x*" -or
                $_.ProviderName -like "*Application Error*"
            }

            if ($appErrors) {
                Write-Host ""
                Write-Host "Application Errors Found:" -ForegroundColor Yellow
                foreach ($error in $appErrors) {
                    Write-Host "  Time: $($error.TimeCreated)" -ForegroundColor Yellow
                    Write-Host "  Source: $($error.ProviderName)" -ForegroundColor Yellow
                    Write-Host "  Message: $($error.Message)" -ForegroundColor Yellow
                    Write-Host ""
                }
            } else {
                Write-Host "  No application errors found" -ForegroundColor Green
            }
        } catch {
            Write-Host "  Warning: Could not check Application log" -ForegroundColor Yellow
        }

        Write-Host ""
        Write-Host "========================================" -ForegroundColor Green
        Write-Host "TEST PASSED: Package appears to be working" -ForegroundColor Green
        Write-Host "========================================" -ForegroundColor Green

    } else {
        $exitCode = $Process.ExitCode
        Write-Host ""
        Write-Host "WARNING: Application exited immediately (Exit Code: $exitCode)" -ForegroundColor Red
        Write-Host ""

        # Check for Application Popup events in System log
        Write-Host "Checking System log for Application Popup events..." -ForegroundColor Cyan
        try {
            $popupEvents = Get-WinEvent -FilterHashtable @{
                LogName = 'System'
                ProviderName = 'Application Popup'
                StartTime = $beforeTime
            } -ErrorAction SilentlyContinue

            if ($popupEvents) {
                Write-Host ""
                Write-Host "====================================" -ForegroundColor Red
                Write-Host "MISSING DLL ERROR DETECTED!" -ForegroundColor Red
                Write-Host "====================================" -ForegroundColor Red
                Write-Host ""

                foreach ($event in $popupEvents) {
                    Write-Host "Time: $($event.TimeCreated)" -ForegroundColor Yellow
                    Write-Host "Message:" -ForegroundColor Yellow
                    Write-Host $event.Message -ForegroundColor White
                    Write-Host ""

                    # Try to extract DLL name from message
                    $message = $event.Message
                    if ($message -match '([A-Za-z0-9_\-]+\.dll)') {
                        $missingDll = $Matches[1]
                        Write-Host "MISSING DLL: $missingDll" -ForegroundColor Red
                        Write-Host ""

                        # Check if it exists in Engine binaries
                        $engineBin = Join-Path (Split-Path -Parent (Split-Path -Parent (Split-Path -Parent $PSScriptRoot))) "Binaries\Win64"
                        $sourcePath = Join-Path $engineBin $missingDll

                        if (Test-Path $sourcePath) {
                            $size = (Get-Item $sourcePath).Length / 1MB
                            Write-Host "FOUND in Engine/Binaries/Win64:" -ForegroundColor Green
                            Write-Host "  Path: $sourcePath" -ForegroundColor White
                            Write-Host "  Size: $("{0:N2}" -f $size) MB" -ForegroundColor White
                            Write-Host ""
                            Write-Host "ACTION REQUIRED:" -ForegroundColor Yellow
                            Write-Host "  1. Add '$missingDll' to the RequiredModules list in PackageForDistribution.ps1" -ForegroundColor Yellow
                            Write-Host "  2. Re-run PackageForDistribution.bat" -ForegroundColor Yellow
                            Write-Host "  3. Re-run this test" -ForegroundColor Yellow
                        } else {
                            Write-Host "NOT FOUND in Engine/Binaries/Win64" -ForegroundColor Red
                            Write-Host "This DLL may need to be obtained from another source." -ForegroundColor Yellow
                        }
                    }
                }
            } else {
                Write-Host "  No Application Popup events found" -ForegroundColor Yellow
            }
        } catch {
            Write-Host "  Could not access System log" -ForegroundColor Yellow
        }

        # Check Application log
        Write-Host ""
        Write-Host "Checking Application log for errors..." -ForegroundColor Cyan
        try {
            $errors = Get-WinEvent -FilterHashtable @{
                LogName = 'Application'
                Level = 2  # Error
                StartTime = $beforeTime
            } -MaxEvents 10 -ErrorAction SilentlyContinue | Where-Object { $_.Message -like "*IoStoreDependencyViewer*" -or $_.Message -like "*.dll*" }

            if ($errors) {
                Write-Host ""
                Write-Host "Recent Application Errors:" -ForegroundColor Red
                foreach ($error in $errors) {
                    Write-Host "  Time: $($error.TimeCreated)" -ForegroundColor Yellow
                    Write-Host "  Message: $($error.Message)" -ForegroundColor Yellow
                    Write-Host ""
                }
            }
        } catch {
            Write-Host "  Could not access Application log" -ForegroundColor Yellow
        }

        Write-Host ""
        Write-Host "========================================" -ForegroundColor Red
        Write-Host "TEST FAILED: Application crashed on startup" -ForegroundColor Red
        Write-Host "========================================" -ForegroundColor Red

        exit 1
    }
} catch {
    Write-Host ""
    Write-Host "ERROR: Failed to launch application" -ForegroundColor Red
    Write-Host $_.Exception.Message -ForegroundColor Red
    Write-Host ""

    Write-Host "========================================" -ForegroundColor Red
    Write-Host "TEST FAILED: Could not start process" -ForegroundColor Red
    Write-Host "========================================" -ForegroundColor Red

    exit 1
}

Write-Host ""
Write-Host "Test directory: $TestDir" -ForegroundColor Cyan
Write-Host "(Preserved for manual inspection if needed)" -ForegroundColor Cyan
Write-Host ""
